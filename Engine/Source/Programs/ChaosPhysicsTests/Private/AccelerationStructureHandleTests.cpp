// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"
#include "ChaosTestScene.h"

#include "Chaos/Box.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace Chaos
{
	void SetShapeFilter(FSingleParticlePhysicsProxy* Proxy, const int32 ShapeIndex, uint8 CollisionChannelIndex, Chaos::EFilterFlags Flags, uint32 BlockMask, uint32 OverlapMask)
	{
		Filter::FShapeFilterBuilder Builder;
		Builder.SetCollisionChannelIndex(CollisionChannelIndex);
		Builder.SetBlockChannelMask(BlockMask);
		Builder.SetOverlapChannelMask(OverlapMask);
		Builder.SetFilterFlags(Flags);
		const Filter::FShapeFilterData FilterData = Builder.Build();

		Proxy->GetGameThreadAPI().SetShapeFilterData(ShapeIndex, FilterData);
	}

	void SetShapeEnabled(FSingleParticlePhysicsProxy* Proxy, const int32 ShapeIndex, bool bSimEnabled, bool bQueryEnabled)
	{
		FRigidBodyHandle_External& BodyHandle = Proxy->GetGameThreadAPI();
		BodyHandle.SetShapeSimCollisionEnabled(ShapeIndex, bSimEnabled);
		BodyHandle.SetShapeQueryCollisionEnabled(ShapeIndex, bQueryEnabled);
	}

	bool PrePreQueryFilter(FSingleParticlePhysicsProxy* Proxy, uint8 CollisionChannelIndex, const uint32 BlockMask)
	{
		Chaos::Filter::FQueryTraceFilterBuilder QueryBuilder;
		QueryBuilder.SetCollisionChannelIndex(CollisionChannelIndex);
		QueryBuilder.SetBlockChannelMask(BlockMask);
		const Chaos::Filter::FQueryFilterData QueryFilter = QueryBuilder.Build();

		FAccelerationStructureHandle ParticleHandle(Proxy->GetParticle_LowLevel());
		return ParticleHandle.PrePreFilter(QueryFilter);
	}

	bool PrePreSimFilter(FSingleParticlePhysicsProxy* Proxy, uint8 CollisionChannelIndex, const uint32 BlockMask)
	{
		Chaos::Filter::FShapeFilterBuilder Builder;
		Builder.SetCollisionChannelIndex(CollisionChannelIndex);
		Builder.SetBlockChannelMask(BlockMask);

		Chaos::Filter::FShapeUnionFilterData ParticleFilter;
		ParticleFilter.Combine(Builder.Build());

		FAccelerationStructureHandle ParticleHandle(Proxy->GetParticle_LowLevel());
		return ParticleHandle.PrePreFilter(ParticleFilter);
	}

	TEST_CASE("FAccelerationStructureHandle - PrePreFilter", "[Chaos][unit]")
	{
		using namespace Chaos::LowLevelTest;

		FChaosTestScene Scene;

		const FVector3d HalfExtents(50, 50, 50);
		TArray<FImplicitObjectPtr> Geometry
		{
			FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents)),
			FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents)),
			FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents)),
			FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents)),
		};
		FImplicitObjectPtr UnionImplicit = FImplicitObjectPtr(new FImplicitObjectUnion(MoveTemp(Geometry)));
		FSingleParticlePhysicsProxy* UnionProxy0 = Scene.AddDynamicProxy(TEXT("Union0"), FTransform3d(FVector3d(0, 0, 0)), UnionImplicit);		
		
		for (int32 ShapeIndex = 0; ShapeIndex < 4; ++ShapeIndex)
		{
			const int32 CollisionChannelIndex = ShapeIndex;
			// Setup the block and overlap masks with a gap in-between.
			const int32 BlockMask = 1 << ShapeIndex;
			const int32 OverlapMask = 1 << (ShapeIndex + 5);
			SetShapeFilter(UnionProxy0, ShapeIndex, CollisionChannelIndex, EFilterFlags::All, BlockMask, OverlapMask);
		}

		SECTION("Basic")
		{
			// Sim: Channels(0b1111) Block(0b1111)
			// Query: Channels(0b1111) Block(0b1111) Overlap(0b111100000)
			
			// Query only uses the channel index. Test all values.
			CHECK(false == PrePreQueryFilter(UnionProxy0, 0, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 1, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 2, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 3, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 4, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 5, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 6, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 7, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 8, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 9, 0));

			// Sim uses both.
			// First test all valid collision channels with a block mask that passes all shapes.
			CHECK(false == PrePreSimFilter(UnionProxy0, 0, 0b1111));
			CHECK(false == PrePreSimFilter(UnionProxy0, 1, 0b1111));
			CHECK(false == PrePreSimFilter(UnionProxy0, 2, 0b1111));
			CHECK(false == PrePreSimFilter(UnionProxy0, 3, 0b1111));
			// Now test all valid block masks with a collision channel index that passes all.
			CHECK(false == PrePreSimFilter(UnionProxy0, 0, 0b0001));
			CHECK(false == PrePreSimFilter(UnionProxy0, 0, 0b0010));
			CHECK(false == PrePreSimFilter(UnionProxy0, 0, 0b0100));
			CHECK(false == PrePreSimFilter(UnionProxy0, 0, 0b1000));

			// Now test edge cases that will be filtered.
			CHECK(true == PrePreSimFilter(UnionProxy0, 0, 0b0000));
			CHECK(true == PrePreSimFilter(UnionProxy0, 0, 0b10000));
			CHECK(true == PrePreSimFilter(UnionProxy0, 4, 0b1111));
		}
		SECTION("Mixed Sim/Query Enabled")
		{
			SetShapeEnabled(UnionProxy0, 0, false, false);
			SetShapeEnabled(UnionProxy0, 1, false, true);
			SetShapeEnabled(UnionProxy0, 2, true, false);
			SetShapeEnabled(UnionProxy0, 3, true, true);
			// Sim: Channels(0b1100) Block(0b1100)
			// Query: Channels(0b1010) Block(0b1010) Overlap(0b101000000) Combined(0b101001010)

			// Test all query channel indices.
			CHECK(true == PrePreQueryFilter(UnionProxy0, 0, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 1, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 2, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 3, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 4, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 5, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 6, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 7, 0));
			CHECK(false == PrePreQueryFilter(UnionProxy0, 8, 0));
			CHECK(true == PrePreQueryFilter(UnionProxy0, 9, 0));

			// Test all channel indices.
			CHECK(true == PrePreSimFilter(UnionProxy0, 0, 0b1111));
			CHECK(true == PrePreSimFilter(UnionProxy0, 1, 0b1111));
			CHECK(false == PrePreSimFilter(UnionProxy0, 2, 0b1111));
			CHECK(false == PrePreSimFilter(UnionProxy0, 3, 0b1111));
			CHECK(true == PrePreSimFilter(UnionProxy0, 4, 0b1111));

			// Test all block masks.
			CHECK(true == PrePreSimFilter(UnionProxy0, 2, 0b0001));
			CHECK(true == PrePreSimFilter(UnionProxy0, 2, 0b0010));
			CHECK(false == PrePreSimFilter(UnionProxy0, 2, 0b0100));
			CHECK(false == PrePreSimFilter(UnionProxy0, 2, 0b1000));
			CHECK(true == PrePreSimFilter(UnionProxy0, 2, 0b10000));
		}
	}
} // namespace Chaos
