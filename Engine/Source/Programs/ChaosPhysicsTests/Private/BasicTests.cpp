// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "Chaos/AABB.h"

// Simple test cases
TEST_CASE("AABB Construction", "[Chaos][unit]")
{
	using namespace Chaos;
	FVector HalfExtent{ 100.0f, 100.0f, 100.0f };
	FAABB3 DefaultAabb;
	FAABB3 InitializedAabb(-HalfExtent, HalfExtent);

	REQUIRE(DefaultAabb.IsEmpty());
	REQUIRE(!InitializedAabb.IsEmpty());
	REQUIRE(InitializedAabb.GetVolume() == 8000000);
}

TEST_CASE("AABB Intersection", "[Chaos][unit]")
{
	using namespace Chaos;

	FAABB3 A{ {-100.0f, -100.0f, -100.0f}, {10.0f, 10.0f, 10.0f} };
	FAABB3 B{ {-10.0f, -10.0f, -10.0f}, {100.0f, 100.0f, 100.0f} };
	FAABB3 Intersect = A.GetIntersection(B);

	REQUIRE(Intersect.Min() == FVector{ -10.0f, -10.0f, -10.0f });
	REQUIRE(Intersect.Max() == FVector{ 10.0f, 10.0f, 10.0f });
}

// Test with Fixtures
TEST_CASE("AABB Operations", "[Chaos][unit]")
{
	// Rather than use a fixture class, this portion runs for
	// all sections below, each section can mutate and use the 
	// state without interfering with each other
	using namespace Chaos;
	FVector HalfExtent{ 100.0f, 100.0f, 100.0f };
	FAABB3 TestAabb(-HalfExtent, HalfExtent);
	const float InitialVolume = TestAabb.GetVolume();

	SECTION("Growing around another box expands")
	{
		TestAabb.GrowToInclude({{-200, -200, -200}, {200, 200, 200}});
		const float NewVolume = TestAabb.GetVolume();
		REQUIRE(NewVolume > InitialVolume);
	}

	SECTION("Growing around a fully contained box does not expand")
	{
		TestAabb.GrowToInclude({ {-20, -20, -20}, {20, 20, 20} });
		const float NewVolume = TestAabb.GetVolume();
		REQUIRE(NewVolume == InitialVolume);
	}
}

// BDD Style test matching above
SCENARIO("AABBs can grow to include larger AABBs but not grow if an enclosed AABB is provided", "[Chaos][unit]")
{
	GIVEN("A 200x200x200 AABB")
	{
		using namespace Chaos;
		FVector HalfExtent{ 100.0f, 100.0f, 100.0f };
		FAABB3 TestAabb(-HalfExtent, HalfExtent);
		const float InitialVolume = TestAabb.GetVolume();

		REQUIRE(TestAabb.GetVolume() == 8000000);

		WHEN("It grows around a larger AABB")
		{
			TestAabb.GrowToInclude({ {-200, -200, -200}, {200, 200, 200} });

			THEN("Its volume increases")
			{
				REQUIRE(TestAabb.GetVolume() > InitialVolume);
			}
		}

		WHEN("It grows around a smaller AABB")
		{
			TestAabb.GrowToInclude({ {-20, -20, -20}, {20, 20, 20} });

			THEN("Its volume remains the same")
			{
				REQUIRE(TestAabb.GetVolume() == InitialVolume);
			}
		}
	}
}

// Benchmark
TEST_CASE("AABB Transform Benchmark", "[Chaos][!benchmark]")
{
	using namespace Chaos;

	FTransform Transform(FRotator{ 90, 0, 0 }, FVector{ 0, 0, 0 });
	FAABB3 Aabb{ {100, 100, 100}, {200, 200, 200} };

	BENCHMARK("Rotate X 90")
	{
		return Aabb.TransformedAABB(Transform);
	};
}