// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"

#include "ChaosTestHarness.h"
#include <catch2/generators/catch_generators_range.hpp>

#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/ChaosArchive.h"
#include "Chaos/LowLevelTest/ChaosTestErrorLogSuppressor.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

namespace Chaos::Tests
{
	using FMaskFilter = uint8;

	// Helper to output the serialization data to a code string so we can easily hardcode the test
	FString ToString(const TArray<uint8>& DataArray)
	{
		TStringBuilder<128> Builder;
		Builder.Append("{");
		for (const uint8 Data : DataArray)
		{
			if (Data != 0)
			{
				Builder.Appendf(TEXT("0x%X, "), Data);
			}
			else
			{
				Builder.Appendf(TEXT("%X, "), Data);
			}
		}
		Builder.Append("}");
		return Builder.ToString();
	}

	void Save(FCollisionData& InCollisionData, const int32 Version, TArray<uint8>& OutData)
	{
		FMemoryWriter Ar(OutData);
		FChaosArchive Writer(Ar);
		Writer.SetCustomVersion(FFortniteMainBranchObjectVersion::GUID, Version, FName("Unused"));
		Writer.SetShouldSkipUpdateCustomVersion(true);
		Writer << InCollisionData;
	}

	void Load(const TArray<uint8>& InData, const int32 Version, FCollisionData& OutCollisionData)
	{
		FMemoryReader Ar(InData);
		FChaosArchive Reader(Ar);
		Reader.SetCustomVersion(FFortniteMainBranchObjectVersion::GUID, Version, FName("Unused"));
		Reader << OutCollisionData;
	}

	uint32 BuildLegacyWord3(const FMaskFilter MaskFilter, const uint8 CollisionIndex, const EFilterFlags Flags)
	{
		constexpr uint32 NumBitsFlags = 21;
		constexpr uint32 NumBitsCollisionChannel = 5;
		constexpr uint32 NumBitsMaskFilter = 6;
		const uint32 FlagsMask = (1 << NumBitsFlags) - 1;
		const uint32 CollisionChannelMask = (1 << NumBitsCollisionChannel) - 1;
		const uint32 MaskFilterMask = (1 << NumBitsMaskFilter) - 1;

		uint32 Result = 0;
		Result |= (uint32)Flags & FlagsMask;
		Result |= ((uint32)CollisionIndex & CollisionChannelMask) << NumBitsFlags;
		Result |= ((uint32)MaskFilter & MaskFilterMask) << (NumBitsFlags + NumBitsCollisionChannel);
		return Result;
	}

	FCollisionFilterData BuildLegacyQueryFilter(const uint32 OwnerId, const uint32 BlockMask, const uint32 OverlapMask, const FMaskFilter MaskFilter, const uint8 CollisionIndex, const EFilterFlags Flags)
	{
		FCollisionFilterData Result;
		Result.Word0 = OwnerId;
		Result.Word1 = BlockMask;
		Result.Word2 = OverlapMask;
		Result.Word3 = BuildLegacyWord3(MaskFilter, CollisionIndex, Flags);
		return Result;
	}

	FCollisionFilterData BuildLegacySimFilter(const uint32 BodyIndex, const uint32 BlockMask, const uint32 ComponentId, const FMaskFilter MaskFilter, const uint8 CollisionIndex, const EFilterFlags Flags)
	{
		FCollisionFilterData Result;
		Result.Word0 = BodyIndex;
		Result.Word1 = BlockMask;
		Result.Word2 = ComponentId;
		Result.Word3 = BuildLegacyWord3(MaskFilter, CollisionIndex, Flags);
		return Result;
	}

	struct FCombinedTestData
	{
		uint32 BodyIndex = 0;
		uint32 ComponentId = 0;
		uint32 OwnerId = 0;
		uint64 OverlapMask = 0;
		uint64 BlockMask = 0;
		FMaskFilter MaskFilter = 0;
		uint8 CollisionChannel = 0;
		EFilterFlags Flags = EFilterFlags::None;

		static_assert(PLATFORM_LITTLE_ENDIAN, "Serialization needs updating to support big-endian platforms!");
		TArray<uint8> SerializationData;

		void SaveLegacy(const int32 Version)
		{
			FCollisionData CollisionData;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CollisionData.SimData = BuildLegacySimFilter(BodyIndex, (uint32)BlockMask, ComponentId, MaskFilter, CollisionChannel, Flags);
			CollisionData.QueryData = BuildLegacyQueryFilter(OwnerId, (uint32)BlockMask, (uint32)OverlapMask, MaskFilter, CollisionChannel, Flags);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Chaos::Tests::Save(CollisionData, Version, SerializationData);
		}

		void Save(const int32 Version)
		{
			Chaos::Filter::FShapeFilterBuilder Builder;
			Builder.SetOverlapChannelMask(OverlapMask);
			Builder.SetBlockChannelMask(BlockMask);
			Builder.SetCollisionChannelIndex(CollisionChannel);
			Builder.SetMaskFilter(MaskFilter);
			Builder.SetFilterFlags(Flags);

			FCollisionData CollisionData;
			CollisionData.SetFilterInstanceData(Chaos::Filter::FInstanceData(OwnerId, ComponentId));
			CollisionData.SetShapeFilterData(Builder.Build());

			Chaos::Tests::Save(CollisionData, Version, SerializationData);
		}

		FCollisionData Load(const int32 Version) const
		{
			FCollisionData ResultCollisionData;
			Chaos::Tests::Load(SerializationData, Version, ResultCollisionData);
			return ResultCollisionData;
		}
	};

	struct FLegacyFilterTestData
	{
		uint32 BodyIndex = 0;
		uint32 ComponentId = 0;
		uint32 OwnerId = 0;

		uint32 QueryOverlapMask = 0;
		uint32 QueryBlockMask = 0;
		uint32 SimBlockMask = 0;
		uint32 ExpectedBlockMask = 0;

		FMaskFilter QueryMaskFilter = 0;
		FMaskFilter SimMaskFilter = 0;
		FMaskFilter ExpectedMaskFilter = 0;

		uint8 QueryCollisionChannel = 0;
		uint8 SimCollisionChannel = 0;
		uint8 ExpectedCollisionChannel = 0;

		EFilterFlags QueryFlags = EFilterFlags::None;
		EFilterFlags SimFlags = EFilterFlags::None;
		EFilterFlags ExpectedFlags = EFilterFlags::None;

		static_assert(PLATFORM_LITTLE_ENDIAN, "Serialization needs updating to support big-endian platforms!");
		TArray<uint8> SerializationData;

		void Save(const int32 Version)
		{
			FCollisionData CollisionData;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CollisionData.QueryData = BuildLegacyQueryFilter(OwnerId, QueryBlockMask, QueryOverlapMask, QueryMaskFilter, QueryCollisionChannel, QueryFlags);
			CollisionData.SimData = BuildLegacySimFilter(BodyIndex, SimBlockMask, ComponentId, SimMaskFilter, SimCollisionChannel, SimFlags);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Chaos::Tests::Save(CollisionData, Version, SerializationData);
		}

		FCollisionData Load(const int32 Version) const
		{
			FCollisionData ResultCollisionData;
			Chaos::Tests::Load(SerializationData, Version, ResultCollisionData);
			return ResultCollisionData;
		}
	};

	TEST_CASE("FCollisionData: Collision Filters Round Trip", "[Chaos][CollisionData][unit]")
	{
		const Chaos::Filter::FInstanceData InstanceData(1, 2);
		Chaos::Filter::FShapeFilterBuilder Builder;
		Builder.SetBlockChannelMask(0xFEDCBA976543210);
		Builder.SetOverlapChannelMask(0x0123456789ABCDEF);
		Builder.SetMaskFilter(33); // 0b100001
		Builder.SetCollisionChannelIndex(63);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision | EFilterFlags::ContactNotify);
		const Chaos::Filter::FShapeFilterData ShapeFilter = Builder.Build();

		FCollisionData CollisionData;
		CollisionData.SetFilterInstanceData(InstanceData);
		CollisionData.SetShapeFilterData(ShapeFilter);
		const Chaos::Filter::FInstanceData ResultInstanceData = CollisionData.GetFilterInstanceData();
		const Chaos::Filter::FShapeFilterData ResultShapeFilter = CollisionData.GetShapeFilterData();

		CHECK(ResultInstanceData.GetOwnerId() == InstanceData.GetOwnerId());
		CHECK(ResultInstanceData.GetComponentId() == InstanceData.GetComponentId());
		CHECK(ResultShapeFilter.GetOverlapChannels() == ShapeFilter.GetOverlapChannels());
		CHECK(ResultShapeFilter.GetBlockChannels() == ShapeFilter.GetBlockChannels());
		CHECK(ResultShapeFilter.GetMaskFilter() == ShapeFilter.GetMaskFilter());
		CHECK(ResultShapeFilter.GetCollisionChannelIndex() == ShapeFilter.GetCollisionChannelIndex());
		CHECK(ResultShapeFilter.GetCollisionChannelMask() == ShapeFilter.GetCollisionChannelMask());
		CHECK(ResultShapeFilter.GetFlags() == ShapeFilter.GetFlags());
	}

	TEST_CASE("FCollisionData: CollisionFilter Serialization: Pre 64Bit -> 64Bit: Combined", "[Chaos][CollisionData][unit]")
	{
		constexpr int32 Version = FFortniteMainBranchObjectVersion::CollisionFilter64Bit - 1;

		TArray<FCombinedTestData> Tests
		{
			FCombinedTestData
			{
				.OwnerId = 0x12345678,
				.SerializationData = TArray<uint8>{0x78, 0x56, 0x34, 0x12, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.OverlapMask = 0x12345678,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.BlockMask = 0x12345678,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.ComponentId = 0x12345678,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 0, 0, 0, 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.Flags = EFilterFlags::All, // Word3: 000000 00000 000000000000011111111 -> 0x0000 00FF
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0xFF, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0xFF, 0, 0, 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.MaskFilter = 0x21, // Word3: 100001 00000 000000000000000000000 -> 0x8400 0000
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x84, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x84, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.CollisionChannel = 31, // Word3: 000000 11111 000000000000000000000 -> 0x03E0 0000
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0xE0, 0x03, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0xE0, 0x03, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
			FCombinedTestData
			{
				.ComponentId = 1,
				.OwnerId = 2,
				.OverlapMask = 0xABCD4567,
				.BlockMask = 0x12345678,
				.MaskFilter = 0x21,
				.CollisionChannel = 31,
				.Flags = EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision | EFilterFlags::ContactNotify,
				.SerializationData = TArray<uint8>{2, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 0x67, 0x45, 0xCD, 0xAB, 0x0B, 0, 0xE0, 0x87, 0, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 1, 0, 0, 0, 0x0B, 0, 0xE0, 0x87, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, },
			},
		};
		for (int32 I = 0; I < Tests.Num(); ++I)
		{
			CAPTURE(I);
			const FCombinedTestData& TestData = Tests[I];
			const FCollisionData ResultCollisionData = TestData.Load(Version);

			const Chaos::Filter::FInstanceData& InstanceData = ResultCollisionData.GetFilterInstanceData();
			const Chaos::Filter::FShapeFilterData& ShapeFilter = ResultCollisionData.GetShapeFilterData();

			CHECK(TestData.OwnerId == InstanceData.GetOwnerId());
			CHECK(TestData.ComponentId == InstanceData.GetComponentId());
			CHECK(TestData.OverlapMask == ShapeFilter.GetOverlapChannels());
			CHECK(TestData.BlockMask == ShapeFilter.GetBlockChannels());
			CHECK(TestData.MaskFilter == ShapeFilter.GetMaskFilter());
			CHECK(TestData.CollisionChannel == ShapeFilter.GetCollisionChannelIndex());
			CHECK(TestData.Flags == ShapeFilter.GetFlags());
		}
	}

	TEST_CASE("FCollisionData: CollisionFilter Serialization: Pre 64Bit -> 64Bit: Complex", "[Chaos][CollisionData][unit]")
	{
		Chaos::LowLevelTest::FErrorLogSuppressor Suppressor;
		Suppressor.ExpectWarning(TEXT("Deserializing collision filters with different"));

		constexpr int32 Version = FFortniteMainBranchObjectVersion::CollisionFilter64Bit - 1;

		TArray<FLegacyFilterTestData> Tests
		{
			FLegacyFilterTestData
			{
				.QueryBlockMask = 0x12345678,
				.SimBlockMask = 0xAABCDEFF,
				.ExpectedBlockMask = 0x12345678,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0x78, 0x56, 0x34, 0x12, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0xFF, 0xDE, 0xBC, 0xAA, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1, 0, 0, 0, 0, 0, },
			},
			FLegacyFilterTestData
			{
				.QueryCollisionChannel = 1,
				.SimCollisionChannel = 2,
				.ExpectedCollisionChannel = 1,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x20, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x40, 0, 0x1, 0x1, 0, 0, 0, 0, 0, },
			},
			FLegacyFilterTestData
			{
				.QueryFlags = EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, // 0x3
				.SimFlags = EFilterFlags::ContactNotify, // 0xA
				.ExpectedFlags = EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision | EFilterFlags::ContactNotify,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x3, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x8, 0, 0, 0, 0x1, 0x1, 0, 0, 0, 0, 0, },
			},
			FLegacyFilterTestData
			{
				.QueryFlags = EFilterFlags::ContactNotify, // 0xA
				.SimFlags = EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, // 0x3
				.ExpectedFlags = EFilterFlags::None,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x8, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x3, 0, 0, 0, 0x1, 0x1, 0, 0, 0, 0, 0, },
			},
			FLegacyFilterTestData
			{
				.QueryMaskFilter = 0xA, // Word3: 001010 00000 000000000000000000000 -> 0x2800 0000
				.SimMaskFilter = 0xF, // Word3: 001111 00000 000000000000000000000 -> 0x3C00 0000
				.ExpectedMaskFilter = 0xA,
				.SerializationData = TArray<uint8>{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x28, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0x3C, 1, 1, 0, 0, 0, 0, 0, },
			},
		};
		for (int32 I = 0; I < Tests.Num(); ++I)
		{
			CAPTURE(I);
			const FLegacyFilterTestData& TestData = Tests[I];
			const FCollisionData ResultCollisionData = TestData.Load(Version);

			const Chaos::Filter::FInstanceData& InstanceData = ResultCollisionData.GetFilterInstanceData();
			const Chaos::Filter::FShapeFilterData& ShapeFilter = ResultCollisionData.GetShapeFilterData();

			CHECK(TestData.OwnerId == InstanceData.GetOwnerId());
			CHECK(TestData.ComponentId == InstanceData.GetComponentId());
			CHECK(TestData.QueryOverlapMask == ShapeFilter.GetOverlapChannels());
			CHECK(TestData.ExpectedBlockMask == ShapeFilter.GetBlockChannels());
			CHECK(TestData.ExpectedMaskFilter == ShapeFilter.GetMaskFilter());
			CHECK(TestData.ExpectedCollisionChannel == ShapeFilter.GetCollisionChannelIndex());
			CHECK(TestData.ExpectedFlags == ShapeFilter.GetFlags());
		}
	}

	TEST_CASE("FCollisionData: CollisionFilter Serialization: Round Trip", "[Chaos][CollisionData][unit]")
	{
		constexpr int32 Version = FFortniteMainBranchObjectVersion::CollisionFilter64Bit;

		TArray<FCombinedTestData> Tests
		{
			FCombinedTestData
			{
				.OwnerId = 0x12345678,
			},
			FCombinedTestData
			{
				.OverlapMask = 0x12345678,
			},
			FCombinedTestData
			{
				.OverlapMask = 0xFEDCBA976543210,
			},
			FCombinedTestData
			{
				.BlockMask = 0x12345678,
			},
			FCombinedTestData
			{
				.BlockMask = 0xFEDCBA976543210,
			},
			FCombinedTestData
			{
				.ComponentId = 0x12345678,
			},
			FCombinedTestData
			{
				.Flags = EFilterFlags::All,
			},
			FCombinedTestData
			{
				.MaskFilter = 0x21,
			},
			FCombinedTestData
			{
				.CollisionChannel = 31,
			},
			FCombinedTestData
			{
				.CollisionChannel = 63,
			},
			FCombinedTestData
			{
				.ComponentId = 1,
				.OwnerId = 2,
				.OverlapMask = 0xFEDCBA9876543210,
				.BlockMask = 0x123456789ABCDEF,
				.MaskFilter = 31,
				.CollisionChannel = 63,
				.Flags = EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision | EFilterFlags::ContactNotify,
			},
		};
		for (int32 I = 0; I < Tests.Num(); ++I)
		{
			CAPTURE(I);
			FCombinedTestData& TestData = Tests[I];

			TestData.Save(Version);
			const FCollisionData ResultCollisionData = TestData.Load(Version);

			const Chaos::Filter::FInstanceData& InstanceData = ResultCollisionData.GetFilterInstanceData();
			const Chaos::Filter::FShapeFilterData& ShapeFilter = ResultCollisionData.GetShapeFilterData();

			CHECK(TestData.OwnerId == InstanceData.GetOwnerId());
			CHECK(TestData.ComponentId == InstanceData.GetComponentId());
			CHECK(TestData.OverlapMask == ShapeFilter.GetOverlapChannels());
			CHECK(TestData.BlockMask == ShapeFilter.GetBlockChannels());
			CHECK(TestData.MaskFilter == ShapeFilter.GetMaskFilter());
			CHECK(TestData.CollisionChannel == ShapeFilter.GetCollisionChannelIndex());
			CHECK(TestData.Flags == ShapeFilter.GetFlags());
		}
	}

	TEST_CASE("CollisionData: Storage: Perf", "[Chaos][CollisionData][!benchmark]")
	{
		constexpr uint64 Iterations = 100000;
		TArray<Chaos::Filter::FShapeFilterData> ShapeFilters;
		TArray<Chaos::Filter::FInstanceData> InstanceDatas;
		TArray<Chaos::Filter::FCombinedShapeFilterData> CombinedDatas;
		ShapeFilters.SetNum(Iterations);
		InstanceDatas.SetNum(Iterations);
		CombinedDatas.SetNum(Iterations);
		FCollisionData CollisionData;

		BENCHMARK("Get ShapeFilterData")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				ShapeFilters[I] = CollisionData.GetShapeFilterData();
			}
		};
		BENCHMARK("Get InstanceData")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				InstanceDatas[I] = CollisionData.GetFilterInstanceData();
			}
		};
		BENCHMARK("Get CombinedFilter")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				CombinedDatas[I] = CollisionData.GetCombinedShapeFilterData();
			}
		};
		BENCHMARK("Set ShapeFilterData")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				CollisionData.SetShapeFilterData(ShapeFilters[I]);
			}
		};
		BENCHMARK("Set InstanceData")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				CollisionData.SetFilterInstanceData(InstanceDatas[I]);
			}
		};
		BENCHMARK("Set CombinedFilter")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				CollisionData.SetCombinedShapeFilterData(CombinedDatas[I]);
			}
		};
		BENCHMARK("GetSet ShapeFilterData")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				const Chaos::Filter::FShapeFilterData ShapeData = ShapeFilters[I];
				ShapeFilters[I] = CollisionData.GetShapeFilterData();
				CollisionData.SetShapeFilterData(ShapeData);
			}
		};
		BENCHMARK("GetSet InstanceData")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				const Chaos::Filter::FInstanceData InstanceData = InstanceDatas[I];
				InstanceDatas[I] = CollisionData.GetFilterInstanceData();
				CollisionData.SetFilterInstanceData(InstanceData);
			}
		};
		BENCHMARK("GetSet CombinedFilter")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				const Chaos::Filter::FCombinedShapeFilterData CombinedData = CombinedDatas[I];
				CombinedDatas[I] = CollisionData.GetCombinedShapeFilterData();
				CollisionData.SetCombinedShapeFilterData(CombinedData);
			}
		};
		BENCHMARK("GetSet BothFilters")
		{
			for (uint64 I = 0; I < Iterations; ++I)
			{
				const Chaos::Filter::FShapeFilterData ShapeData = ShapeFilters[I];
				const Chaos::Filter::FInstanceData InstanceData = InstanceDatas[I];

				ShapeFilters[I] = CollisionData.GetShapeFilterData();
				InstanceDatas[I] = CollisionData.GetFilterInstanceData();

				CollisionData.SetShapeFilterData(ShapeData);
				CollisionData.SetFilterInstanceData(InstanceData);
			}
		};
	}
} // namespace Chaos::Tests
