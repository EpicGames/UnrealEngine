// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"
#include "ChaosTestScene.h"

#include "Chaos/Box.h"
#include "Chaos/MassProperties.h"
#include "ChaosSolversModule.h"
#include "EventsData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

TEST_CASE("CollisionEventTests", "[Chaos][Collision][Events][unit]")
{
	using namespace Chaos;
	using namespace Chaos::LowLevelTest;

	const float Dt = 1.0f / 30.0f;

	FChaosTestScene Scene;

	//
	// Collision events are dispatched every frame and contain collision data for awake bodies.
	//
	SECTION("Collision Events dispatched for awake bodies")
	{
		const double KinematicPositionTolerance = UE_SMALL_NUMBER;
		const double DynamicPositionTolerance = UE_KINDA_SMALL_NUMBER;

		// Place a dynamic 100cm box exactly on the floor (a large kinematic box)
		FSingleParticlePhysicsProxy* FloorBox = Scene.AddKinematicBox(TEXT("Floor"), FTransform3d(FVector3d(0, 0, -50)), FVector3d(500, 500, 50));
		FSingleParticlePhysicsProxy* DynamicBox = Scene.AddDynamicBox(TEXT("Box"), FTransform3d(FVector3d(0, 0, 50)), FVector3d(50, 50, 50));

		int NumCallbacks = 0;
		Scene.SetCollisionCallback(
			[&NumCallbacks, DynamicBox](const FCollisionEventData& CollisionEventData)
			{
				// Should have zero or one collision event
				CHECK(CollisionEventData.CollisionData.AllCollisionsArray.Num() >= 0);
				CHECK(CollisionEventData.CollisionData.AllCollisionsArray.Num() <= 1);
				++NumCallbacks;
			});

		// The box should sleep quickly (but we should have at least one tick)
		Scene.TickSolverToSleep(Dt, 50);
		REQUIRE(Scene.IsSceneSleeping());
		CHECK(Scene.GetNumTicks() > 0);

		// We should have one callback for each tick
		CHECK(NumCallbacks == Scene.GetNumTicks());

		// The kinematic box should not have moved at all
		FVector3d FloorBoxPos = FloorBox->GetGameThreadAPI().GetX();
		CHECK_THAT(FloorBoxPos.Z, Catch::Matchers::WithinAbs(-50.0f, KinematicPositionTolerance));

		// The dynamic box should not have moved much
		FVector3d DynamicBoxPos = DynamicBox->GetGameThreadAPI().GetX();
		CHECK_THAT(DynamicBoxPos.Z, Catch::Matchers::WithinAbs(50.0f, DynamicPositionTolerance));
	}


	//
	// The collision event callback should still be called even when there are no collision to report. This
	// is so that user code can track when contact ends.
	//
	SECTION("Collision Events dispatched empty when no awake bodies")
	{
		FSingleParticlePhysicsProxy* FloorBox = Scene.AddKinematicBox(TEXT("Floor"), FTransform3d(FVector3d(0, 0, -50)), FVector3d(500, 500, 50));
		FSingleParticlePhysicsProxy* DynamicBox1 = Scene.AddDynamicBox(TEXT("Box1"), FTransform3d(FVector3d(0, 0, 50)), FVector3d(50, 50, 50));

		// Wait for scene to sleep
		Scene.TickSolverToSleep(Dt, 1000);
		REQUIRE(Scene.IsSceneSleeping());

		// Make sure the box actually went to sleep or the test isn't working...
		REQUIRE(DynamicBox1->GetGameThreadAPI().ObjectState() == EObjectStateType::Sleeping);

		// Attach the collision callback
		int32 NumCallbacks = 0;
		int32 NumCollisions = 0;
		Scene.SetCollisionCallback(
			[&NumCallbacks, &NumCollisions](const FCollisionEventData& CollisionEventData)
			{
				NumCollisions += CollisionEventData.CollisionData.AllCollisionsArray.Num();
				NumCallbacks += 1;
			});

		// Tick the sleeping scene
		Scene.TickSolver(Dt);

		// Callback should have been fired, even though we had no collisions
		CHECK(NumCallbacks == 1);
		CHECK(NumCollisions == 0);

		// If we wake the body we should get collision events again
		DynamicBox1->GetGameThreadAPI().SetObjectState(EObjectStateType::Dynamic);

		NumCallbacks = 0;
		NumCollisions = 0;
		Scene.TickSolver(Dt);

		// Callback should have been called again and we should have 1 collison (Box1-Floor)
		CHECK(NumCallbacks == 1);
		CHECK(NumCollisions == 1);
	}


	//
	// BUG: Collision events were not dispatched on the first frame after waking.
	// Collisions were not being re-added to the ActiveConstraints list when
	// awoken naturally from island modification (from IslandManager). 
	//
	SECTION("Collision Events dispatched empty when no awake bodies")
	{
		FSingleParticlePhysicsProxy* FloorBox = Scene.AddKinematicBox(TEXT("Floor"), FTransform3d(FVector3d(0, 0, -50)), FVector3d(500, 500, 50));
		FSingleParticlePhysicsProxy* DynamicBox1 = Scene.AddDynamicBox(TEXT("Box1"), FTransform3d(FVector3d(0, 0, 50)), FVector3d(50, 50, 50));

		// Wait for sleep
		bool IsSceneSleeping = Scene.TickSolverToSleep(Dt, 1000);
		REQUIRE(IsSceneSleeping);
		REQUIRE(DynamicBox1->GetGameThreadAPI().ObjectState() == EObjectStateType::Sleeping);

		// Attach collision callback
		int32 NumCallbacks = 0;
		int32 NumCollisions = 0;
		Scene.SetCollisionCallback(
			[&NumCallbacks, &NumCollisions](const FCollisionEventData& CollisionEventData)
			{
				NumCollisions += CollisionEventData.CollisionData.AllCollisionsArray.Num();
				NumCallbacks += 1;
			});

		Scene.TickSolver(Dt);
		CHECK(NumCallbacks == 1);
		CHECK(NumCollisions == 0);

		// Spawn a new awake box which is colliding with the sleeping box. This should
		// wake it naturally (as opposed to explicitly like above) which is a separate
		// code path to be tested (in IslandManager.cpp). For this to work, the
		// IslandManager must be adding awakened collisions back into ActiveConstraints.
		FSingleParticlePhysicsProxy* DynamicBox2 = Scene.AddDynamicBox(TEXT("Box2"), FTransform3d(FVector3d(0, 0, 95)), FVector3d(50, 50, 50));

		NumCallbacks = 0;
		NumCollisions = 0;
		Scene.TickSolver(Dt);

		// Callback was called and we have two collisions
		// (Prior to bug fix we only had 1 collision)
		CHECK(NumCallbacks == 1);
		CHECK(NumCollisions == 2);

		// Both boxes should be awake
		CHECK(DynamicBox1->GetGameThreadAPI().ObjectState() != EObjectStateType::Sleeping);
		CHECK(DynamicBox2->GetGameThreadAPI().ObjectState() != EObjectStateType::Sleeping);
	}
}
