// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"

#include "ChaosTestHarness.h"
#include <catch2/generators/catch_generators_range.hpp>

#include "Chaos/CollisionFilterData.h"

namespace Chaos::Filter
{

constexpr Chaos::EFilterFlags EmptyFilterFlags = Chaos::EFilterFlags::None;
constexpr uint64 FullChannelMask = std::numeric_limits<uint64>::max();
constexpr uint64 MaxChannelBitCount = 64;

uint64 ToChannelMask(uint8 ChannelIndex)
{
	return 1LLU << ChannelIndex;
}

FShapeFilterData BuildShapeFilter(const uint8 ChannelIndex, const uint64 BlockMask, const uint64 OverlapMask)
{
	return FShapeFilterBuilder()
		.SetCollisionChannelIndex(ChannelIndex)
		.SetBlockChannelMask(BlockMask)
		.SetOverlapChannelMask(OverlapMask)
		.Build();
}

FShapeUnionFilterData BuildUnionFilter(const TArray<FShapeFilterData>& ShapeFilters)
{
	FShapeUnionFilterData UnionShape;
	for (const FShapeFilterData& ShapeFilter : ShapeFilters)
	{
		UnionShape.Combine(ShapeFilter);
	}
	return UnionShape;
}


TEST_CASE("FInstanceData - Construction", "[Chaos][Filter]")
{
	const uint32 OwnerId = 1;
	const uint32 ComponentId = 2;
	FInstanceData InstanceData(OwnerId, ComponentId);

	CHECK(OwnerId == InstanceData.GetOwnerId());
	CHECK(ComponentId == InstanceData.GetComponentId());
}

TEST_CASE("FInstanceData - IsValid", "[Chaos][Filter]")
{
	FInstanceData InvalidInstanceData;

	CHECK(FInstanceData().IsValid() == false);
	CHECK(FInstanceData(1, 0).IsValid() == true);
	CHECK(FInstanceData(0, 1).IsValid() == true);
	CHECK(FInstanceData(1, 1).IsValid() == true);
}

TEST_CASE("FInstanceData - ToString", "[Chaos][Filter]")
{
	const uint32 OwnerId = 1;
	const uint32 ComponentId = 2;
	FInstanceData InstanceData(OwnerId, ComponentId);

	CHECK(InstanceData.ToString() == FString("OwnerId(1) ComponentId(2)"));
}

TEST_CASE("FShapeFilterData - DefaultConstructor", "[Chaos][Filter]")
{
	FShapeFilterData ShapeFilter;
	CHECK(!ShapeFilter.IsValid());
	CHECK(ShapeFilter.GetCollisionChannelIndex() == 0);
	CHECK(ShapeFilter.GetOverlapChannels() == 0);
	CHECK(ShapeFilter.GetBlockChannels() == 0);
	CHECK(ShapeFilter.GetMaskFilter() == 0);
	CHECK(ShapeFilter.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FShapeFilterData - MaskFilter", "[Chaos][Filter]")
{
	constexpr int8 MaskFilterBitCount = 6;
	constexpr int8 MaskFilter = static_cast<int8>((1 << MaskFilterBitCount) - 1);

	FShapeFilterData ShapeFilter;
	ShapeFilter.SetMaskFilter(MaskFilter);

	const int8 ActualMaskFilter = ShapeFilter.GetMaskFilter();
	CHECK(ActualMaskFilter == MaskFilter);
}

TEST_CASE("FShapeFilterData - IsCollisionChannelSet", "[Chaos][Filter]")
{
	const uint32 ChannelIndex = GENERATE(0, 1, 16, 31, 32, 63);
	CAPTURE(ChannelIndex);

	// TODO @ JoshD: Update for channels as mask.
	FShapeFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(ChannelIndex);
	const FShapeFilterData FilterData = Builder.Build();

	for (uint32 TestIndex = 0; TestIndex < MaxChannelBitCount; ++TestIndex)
	{
		CAPTURE(TestIndex);
		CHECK((TestIndex == ChannelIndex) == FilterData.IsCollisionChannelSet(TestIndex));
	}
}

TEST_CASE("FShapeFilterData - ToString", "[Chaos][Filter]")
{
	FShapeFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(1);
	Builder.SetBlockChannelMask(0xAAAAAAAA);
	Builder.SetOverlapChannelMask(0xCCCCCCCC);
	Builder.SetMaskFilter(0x3F);
	Builder.SetFilterFlags(Chaos::EFilterFlags::SimpleCollision | Chaos::EFilterFlags::ComplexCollision);
	const FShapeFilterData FilterData = Builder.Build();

	CHECK(FilterData.ToString() == "Block(0xAAAAAAAA) Overlap(0xCCCCCCCC) CollisionChannel(0x1) MaskFilter(0x3F) Flags(0x3)");
}

TEST_CASE("FQueryFilterData - DefaultConstructor", "[Chaos][Filter]")
{
	const FQueryFilterData QueryFilterData;
	CHECK(!QueryFilterData.IsValid());
	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::ObjectType);
	CHECK(QueryFilterData.GetObjectTypesToQueryMask() == 0);
	CHECK(QueryFilterData.IsMultiQuery() == false);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryFilterData - IsCollisionChannelSet", "[Chaos][Filter]")
{
	// TODO @ JoshD: Update for channels as mask.
	const uint32 ChannelIndex = GENERATE(0, 1, 16, 31, 32, 63);
	CAPTURE(ChannelIndex);

	FQueryTraceFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(ChannelIndex);
	const FQueryFilterData FilterData = Builder.Build();

	for (uint32 TestIndex = 0; TestIndex < MaxChannelBitCount; ++TestIndex)
	{
		CAPTURE(TestIndex);
		CHECK((TestIndex == ChannelIndex) == FilterData.IsCollisionChannelSet(TestIndex));
	}
}

TEST_CASE("FQueryFilterData::ObjectFilter - ToString", "[Chaos][Filter]")
{
	FQueryObjectFilterBuilder Builder;
	Builder.SetObjectTypes(0xFF00FF);
	Builder.SetMultiQuery(true);
	Builder.SetMaskFilter(0x3F);
	Builder.SetFilterFlags(Chaos::EFilterFlags::SimpleCollision);
	const FQueryFilterData FilterData = Builder.Build();

	CHECK(FilterData.ToString() == "Type(Object) IsMulti(1) ObjectTypes(0xFF00FF) IgnoreMask(0x3F) Flags(0x1)");
}

TEST_CASE("FQueryFilterData::TraceFilter - ToString", "[Chaos][Filter]")
{
	FQueryTraceFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(1);
	Builder.SetBlockChannelMask(0xAAAAAAAA);
	Builder.SetOverlapChannelMask(0xCCCCCCCC);
	Builder.SetMaskFilter(0x3F);
	Builder.SetFilterFlags(Chaos::EFilterFlags::SimpleCollision | Chaos::EFilterFlags::ComplexCollision);
	const FQueryFilterData FilterData = Builder.Build();

	CHECK(FilterData.ToString() == "Type(Channel) Block(0xAAAAAAAA) Overlap(0xCCCCCCCC) ChannelIndex(0x1) IgnoreMask(0x3F) Flags(0x3)");
}

TEST_CASE("FCombinedShapeFilterData - DefaultConstructor", "[Chaos][Filter]")
{
	CHECK(!FCombinedShapeFilterData().IsValid());
}

TEST_CASE("FCombinedShapeFilterData - IsValid", "[Chaos][Filter]")
{
	const FInstanceData InstanceData(1, 2);
	const FShapeFilterData ShapeFilterData = FShapeFilterBuilder().SetCollisionChannelIndex(1).Build();

	CHECK(FCombinedShapeFilterData(ShapeFilterData, FInstanceData()).IsValid());
	CHECK(FCombinedShapeFilterData(FShapeFilterData(), InstanceData).IsValid());
}

TEST_CASE("FCombinedShapeFilterData - InstanceData", "[Chaos][Filter]")
{
	const FInstanceData InstanceData(1, 2);
	FCombinedShapeFilterData CombinedShapeFilterData;
	CombinedShapeFilterData.SetInstanceData(InstanceData);

	const FInstanceData ActualInstanceData = CombinedShapeFilterData.GetInstanceData();
	CHECK(ActualInstanceData.GetOwnerId() == InstanceData.GetOwnerId());
	CHECK(ActualInstanceData.GetComponentId() == InstanceData.GetComponentId());
	CHECK(!CombinedShapeFilterData.GetShapeFilterData().IsValid());
}

TEST_CASE("FCombinedShapeFilterData - ShapeFilterData", "[Chaos][Filter]")
{
	FShapeFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(1);
	Builder.SetOverlapChannelMask(2);
	Builder.SetBlockChannelMask(3);
	Builder.SetMaskFilter(4);
	Builder.SetFilterFlags((Chaos::EFilterFlags)6);
	const FShapeFilterData ShapeFilterData = Builder.Build();
	FCombinedShapeFilterData CombinedShapeFilterData;
	CombinedShapeFilterData.SetShapeFilterData(ShapeFilterData);

	const FShapeFilterData ActualShapeFilterData = CombinedShapeFilterData.GetShapeFilterData();
	CHECK(ActualShapeFilterData.GetCollisionChannelIndex() == ShapeFilterData.GetCollisionChannelIndex());
	CHECK(ActualShapeFilterData.GetOverlapChannels() == ShapeFilterData.GetOverlapChannels());
	CHECK(ActualShapeFilterData.GetBlockChannels() == ShapeFilterData.GetBlockChannels());
	CHECK(ActualShapeFilterData.GetMaskFilter() == ShapeFilterData.GetMaskFilter());
	CHECK(ActualShapeFilterData.GetFlags() == ShapeFilterData.GetFlags());
	CHECK(!CombinedShapeFilterData.GetInstanceData().IsValid());
}

TEST_CASE("FShapeFilterBuilder - LegacyRoundTripConstruction", "[Chaos][Filter]")
{
	// For each section (flags, collision channel, mask filter) test first bit, last bit, all bits set. Throw in one last test for everything being set.
	const uint32 Word3 = GENERATE(0, 0x1, 0x80, 0xFF, 1 << 21, 0x10 << 21, 0x1F << 21, 1 << 26, 0x20 << 26, 0x3F << 26, 0xFFE000FF);

	const uint32 BlockChannels = 2;
	FCollisionFilterData QueryData, SimData;
	QueryData.Word0 = 1;
	QueryData.Word1 = BlockChannels;
	QueryData.Word2 = 3;
	QueryData.Word3 = Word3;
	SimData.Word0 = 0; // BodyIndex is no longer used
	SimData.Word1 = BlockChannels;
	SimData.Word2 = 7;
	SimData.Word3 = Word3;

	FCombinedShapeFilterData CombinedShapeFilter = FShapeFilterBuilder::BuildFromLegacyShapeFilter(QueryData, SimData);

	FCollisionFilterData ResultQueryData, ResultSimData;
	FShapeFilterBuilder::GetLegacyShapeFilter(CombinedShapeFilter, ResultQueryData, ResultSimData);
	CHECK(ResultQueryData.Word0 == QueryData.Word0);
	CHECK(ResultQueryData.Word1 == QueryData.Word1);
	CHECK(ResultQueryData.Word2 == QueryData.Word2);
	CHECK(ResultQueryData.Word3 == QueryData.Word3);
	CHECK(ResultSimData.Word0 == SimData.Word0);
	CHECK(ResultSimData.Word1 == SimData.Word1);
	CHECK(ResultSimData.Word2 == SimData.Word2);
	CHECK(ResultSimData.Word3 == SimData.Word3);
}

TEST_CASE("FShapeFilterBuilder - LegacyRoundTripConstruction - Separate Word3", "[Chaos][Filter]")
{
	const uint32 BlockChannels = 2;
	FCollisionFilterData QueryData, SimData;
	QueryData.Word0 = 1;
	QueryData.Word1 = BlockChannels;
	QueryData.Word2 = 3;
	QueryData.Word3 = 0xFFE00003;
	SimData.Word0 = 0; // BodyIndex is no longer used
	SimData.Word1 = BlockChannels;
	SimData.Word2 = 7;
	SimData.Word3 = 0xFFE0000C;
	const uint32 ExpectedWord3 = 0xFFE0000F;

	FCombinedShapeFilterData CombinedShapeFilter = FShapeFilterBuilder::BuildFromLegacyShapeFilter(QueryData, SimData);

	FCollisionFilterData ResultQueryData, ResultSimData;
	FShapeFilterBuilder::GetLegacyShapeFilter(CombinedShapeFilter, ResultQueryData, ResultSimData);
	CHECK(ResultQueryData.Word0 == QueryData.Word0);
	CHECK(ResultQueryData.Word1 == QueryData.Word1);
	CHECK(ResultQueryData.Word2 == QueryData.Word2);
	CHECK(ResultQueryData.Word3 == ExpectedWord3);
	CHECK(ResultSimData.Word0 == SimData.Word0);
	CHECK(ResultSimData.Word1 == SimData.Word1);
	CHECK(ResultSimData.Word2 == SimData.Word2);
	CHECK(ResultSimData.Word3 == ExpectedWord3);
}

TEST_CASE("FShapeFilterBuilder - RoundTrip to legacy and back", "[Chaos][Filter]")
{
	const uint32 CollisionChannelIndex = GENERATE(0, 1, 31);
	FShapeFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(CollisionChannelIndex);
	Builder.SetBlockChannelMask(2);
	Builder.SetOverlapChannelMask(4);
	Builder.SetMaskFilter(5);
	Builder.SetFilterFlags(EFilterFlags::All);
	const FShapeFilterData ExpectedShapeFilter = Builder.Build();
	const FInstanceData ExpectedInstanceData(9, 10);

	FCollisionFilterData TempQueryData, TempSimData;

	FShapeFilterBuilder::GetLegacyShapeFilter(FCombinedShapeFilterData(ExpectedShapeFilter, ExpectedInstanceData), TempQueryData, TempSimData);
	const FCombinedShapeFilterData CombinedShapeFilter = FShapeFilterBuilder::BuildFromLegacyShapeFilter(TempQueryData, TempSimData);
	const FShapeFilterData ResultShapeFilter = CombinedShapeFilter.GetShapeFilterData();
	const FInstanceData ResultInstanceData = CombinedShapeFilter.GetInstanceData();

	CHECK(ExpectedShapeFilter.GetCollisionChannelIndex() == ResultShapeFilter.GetCollisionChannelIndex());
	CHECK(ExpectedShapeFilter.GetCollisionChannelMask() == ResultShapeFilter.GetCollisionChannelMask());
	CHECK(ExpectedShapeFilter.GetBlockChannels() == ResultShapeFilter.GetBlockChannels());
	CHECK(ExpectedShapeFilter.GetOverlapChannels() == ResultShapeFilter.GetOverlapChannels());
	CHECK(ExpectedShapeFilter.GetMaskFilter() == ResultShapeFilter.GetMaskFilter());
	CHECK(ExpectedShapeFilter.GetFlags() == ResultShapeFilter.GetFlags());
	CHECK(ExpectedInstanceData.GetOwnerId() == ResultInstanceData.GetOwnerId());
	CHECK(ExpectedInstanceData.GetComponentId() == ResultInstanceData.GetComponentId());
}

TEST_CASE("FShapeFilterBuilder - SetLegacyShapeQueryFilter", "[Chaos][Filter]")
{
	FCollisionFilterData QueryData0, QueryData1, SimData;
	QueryData0.Word0 = 1;
	QueryData0.Word1 = 2;
	QueryData0.Word2 = 3;
	QueryData0.Word3 = 0;
	SimData.Word0 = 0; // BodyIndex is no longer used
	SimData.Word1 = 2;
	SimData.Word2 = 7;
	SimData.Word3 = 0xC;
	QueryData1.Word0 = 9;
	QueryData1.Word1 = 10;
	QueryData1.Word2 = 11;
	QueryData1.Word3 = 0xFFE00003;
	const uint32 ExpectedWord1 = 10;
	const uint32 ExpectedWord3 = 0xFFE0000F;

	FCombinedShapeFilterData CombinedShapeFilter = FShapeFilterBuilder::BuildFromLegacyShapeFilter(QueryData0, SimData);
	FShapeFilterBuilder::SetLegacyShapeQueryFilter(CombinedShapeFilter, QueryData1);

	const FCollisionFilterData ResultQueryData = FShapeFilterBuilder::GetLegacyShapeQueryFilter(CombinedShapeFilter);
	const FCollisionFilterData ResultSimData = FShapeFilterBuilder::GetLegacyShapeSimFilter(CombinedShapeFilter);
	CHECK(ResultQueryData.Word0 == QueryData1.Word0);
	CHECK(ResultQueryData.Word1 == ExpectedWord1);
	CHECK(ResultQueryData.Word2 == QueryData1.Word2);
	CHECK(ResultQueryData.Word3 == ExpectedWord3);
	CHECK(ResultSimData.Word0 == SimData.Word0);
	CHECK(ResultSimData.Word1 == ExpectedWord1);
	CHECK(ResultSimData.Word2 == SimData.Word2);
	CHECK(ResultSimData.Word3 == ExpectedWord3);
}

TEST_CASE("FShapeFilterBuilder - SetLegacyShapeSimFilter", "[Chaos][Filter]")
{
	FCollisionFilterData QueryData, SimData0, SimData1;
	QueryData.Word0 = 1;
	QueryData.Word1 = 2;
	QueryData.Word2 = 3;
	QueryData.Word3 = 0x3;
	SimData0.Word0 = 0; // BodyIndex is no longer used
	SimData0.Word1 = 2;
	SimData0.Word2 = 7;
	SimData0.Word3 = 0;
	SimData1.Word0 = 0; // BodyIndex is no longer used
	SimData1.Word1 = 10;
	SimData1.Word2 = 11;
	SimData1.Word3 = 0xFFE0000C;
	const uint32 ExpectedWord1 = 10;
	const uint32 ExpectedWord3 = 0xFFE0000F;

	FCombinedShapeFilterData CombinedShapeFilter = FShapeFilterBuilder::BuildFromLegacyShapeFilter(QueryData, SimData0);
	FShapeFilterBuilder::SetLegacyShapeSimFilter(CombinedShapeFilter, SimData1);

	const FCollisionFilterData ResultQueryData = FShapeFilterBuilder::GetLegacyShapeQueryFilter(CombinedShapeFilter);
	const FCollisionFilterData ResultSimData = FShapeFilterBuilder::GetLegacyShapeSimFilter(CombinedShapeFilter);
	CHECK(ResultQueryData.Word0 == QueryData.Word0);
	CHECK(ResultQueryData.Word1 == ExpectedWord1);
	CHECK(ResultQueryData.Word2 == QueryData.Word2);
	CHECK(ResultQueryData.Word3 == ExpectedWord3);
	CHECK(ResultSimData.Word0 == SimData1.Word0);
	CHECK(ResultSimData.Word1 == ExpectedWord1);
	CHECK(ResultSimData.Word2 == SimData1.Word2);
	CHECK(ResultSimData.Word3 == ExpectedWord3);
}

TEST_CASE("FShapeFilterBuilder - SetCollisionChannelIndex", "[Chaos][Filter]")
{
	const uint8 ChannelIndex = GENERATE(0, 1, 3, 31, 32, 63);

	FShapeFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(ChannelIndex);
	const FShapeFilterData ShapeFilter = Builder.Build();

	CHECK(ShapeFilter.GetCollisionChannelIndex() == ChannelIndex);
	CHECK(ShapeFilter.GetCollisionChannelMask() == ToChannelMask(ChannelIndex));
	CHECK(ShapeFilter.GetOverlapChannels() == 0);
	CHECK(ShapeFilter.GetBlockChannels() == 0);
	CHECK(ShapeFilter.GetMaskFilter() == 0);
	CHECK(ShapeFilter.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FShapeFilterBuilder - SetBlockChannelMask", "[Chaos][Filter]")
{
	const TArray<uint64> BlockChannelMasks{ 0, 1, 2, 3, 0xAAAAAAAA, 0xFFFFFFFF, FullChannelMask };
	const uint32 Index = GENERATE_COPY(Catch::Generators::range(0, BlockChannelMasks.Num()));
	const uint64 BlockChannelMask = BlockChannelMasks[Index];

	FShapeFilterBuilder Builder;
	Builder.SetBlockChannelMask(BlockChannelMask);
	const FShapeFilterData ShapeFilter = Builder.Build();

	CHECK(ShapeFilter.GetCollisionChannelIndex() == 0);
	CHECK(ShapeFilter.GetOverlapChannels() == 0);
	CHECK(ShapeFilter.GetBlockChannels() == BlockChannelMask);
	CHECK(ShapeFilter.GetMaskFilter() == 0);
	CHECK(ShapeFilter.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FShapeFilterBuilder - SetOverlapChannelMask", "[Chaos][Filter]")
{
	const TArray<uint64> OverlapChannelMasks{ 0, 1, 2, 3, 0xAAAAAAAA, 0xFFFFFFFF, FullChannelMask };
	const uint32 Index = GENERATE_COPY(Catch::Generators::range(0, OverlapChannelMasks.Num()));
	const uint64 OverlapChannelMask = OverlapChannelMasks[Index];

	FShapeFilterBuilder Builder;
	Builder.SetOverlapChannelMask(OverlapChannelMask);
	const FShapeFilterData ShapeFilter = Builder.Build();

	CHECK(ShapeFilter.GetCollisionChannelIndex() == 0);
	CHECK(ShapeFilter.GetOverlapChannels() == OverlapChannelMask);
	CHECK(ShapeFilter.GetBlockChannels() == 0);
	CHECK(ShapeFilter.GetMaskFilter() == 0);
	CHECK(ShapeFilter.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FShapeFilterBuilder - SetMaskFilter", "[Chaos][Filter]")
{
	const TArray<uint8> MaskFilters{ 0, 1, 2, 0xF };
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const uint8 MaskFilter = MaskFilters[Index];

	FShapeFilterBuilder Builder;
	Builder.SetMaskFilter(MaskFilter);
	const FShapeFilterData ShapeFilter = Builder.Build();

	CHECK(ShapeFilter.GetCollisionChannelIndex() == 0);
	CHECK(ShapeFilter.GetOverlapChannels() == 0);
	CHECK(ShapeFilter.GetBlockChannels() == 0);
	CHECK(ShapeFilter.GetMaskFilter() == MaskFilter);
	CHECK(ShapeFilter.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FShapeFilterBuilder - SetFilterFlags", "[Chaos][Filter]")
{
	const TArray<EFilterFlags> TestFlags
	{
		EFilterFlags::SimpleCollision,
		EFilterFlags::ComplexCollision,
		EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision,
		EFilterFlags::All,
	};
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const EFilterFlags Flags = TestFlags[Index];

	FShapeFilterBuilder Builder;
	Builder.SetFilterFlags(Flags);
	const FShapeFilterData ShapeFilter = Builder.Build();

	CHECK(ShapeFilter.GetCollisionChannelIndex() == 0);
	CHECK(ShapeFilter.GetOverlapChannels() == 0);
	CHECK(ShapeFilter.GetBlockChannels() == 0);
	CHECK(ShapeFilter.GetMaskFilter() == 0);
	CHECK(ShapeFilter.GetFlags() == Flags);
}

TEST_CASE("FShapeFilterBuilder - SetFilterFlags with bool", "[Chaos][Filter]")
{
	FShapeFilterBuilder Builder;
	SECTION("Given Empty Flags: Set Single Flag")
	{
		Builder.SetFilterFlags(EFilterFlags::ComplexCollision, true);
		CHECK(EFilterFlags::ComplexCollision == Builder.Build().GetFlags());
	}
	SECTION("Given Mixed Flags: Set Multiple Flags ")
	{
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, true);
		CHECK((EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision) == Builder.Build().GetFlags());
	}
	SECTION("Given Empty Flags: Set All Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::All, true);
		CHECK(EFilterFlags::All == Builder.Build().GetFlags());
	}
	SECTION("Given All Flags: Clear Single Flag")
	{
		Builder.SetFilterFlags(EFilterFlags::All);
		Builder.SetFilterFlags(EFilterFlags::ComplexCollision, false);
		const EFilterFlags ExpectedFlags = EFilterFlags::All & ~EFilterFlags::ComplexCollision;
		CHECK(ExpectedFlags == Builder.Build().GetFlags());
	}
	SECTION("Given Mixed Flags: Clear Multiple Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, false);
		CHECK((EFilterFlags::None) == Builder.Build().GetFlags());
	}
	SECTION("Given All Flags: Clear All Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::All);
		Builder.SetFilterFlags(EFilterFlags::All, false);
		CHECK(EFilterFlags::None == Builder.Build().GetFlags());
	}
}

TEST_CASE("FShapeFilterBuilder - BuildBlockAll", "[Chaos][Filter]")
{
	const EFilterFlags Flags = EFilterFlags::ComplexCollision;
	const FShapeFilterData ShapeFilter = FShapeFilterBuilder::BuildBlockAll(Flags);

	CHECK(ShapeFilter.GetCollisionChannelIndex() == 0);
	CHECK(ShapeFilter.GetOverlapChannels() == 0);
	CHECK(ShapeFilter.GetBlockChannels() == FullChannelMask);
	CHECK(ShapeFilter.GetMaskFilter() == 0);
	CHECK(ShapeFilter.GetFlags() == Flags);
}

TEST_CASE("FQueryFilterBuilder - LegacyRoundTripConstruction", "[Chaos][Filter]")
{
	// For each section (flags, collision channel, mask filter) test first bit, last bit, all bits set. Throw in one last test for everything being set.
	const uint32 Word3 = GENERATE(0, 0x1, 0x80, 0xFF, 1 << 21, 0x10 << 21, 0x1F << 21, 1 << 26, 0x20 << 26, 0x3F << 26, 0xFFE000FF);
	FCollisionFilterData QueryData;
	QueryData.Word0 = 1;
	QueryData.Word1 = 2;
	QueryData.Word2 = 3;
	QueryData.Word3 = Word3;

	FQueryFilterData QueryFilterData = FQueryFilterBuilder::BuildFromLegacyQueryFilter(QueryData);
	const FCollisionFilterData ResultQueryData = FQueryFilterBuilder::GetLegacyQueryFilter(QueryFilterData);

	CHECK(ResultQueryData.Word0 == QueryData.Word0);
	CHECK(ResultQueryData.Word1 == QueryData.Word1);
	CHECK(ResultQueryData.Word2 == QueryData.Word2);
	CHECK(ResultQueryData.Word3 == QueryData.Word3);
}

TEST_CASE("FQueryObjectFilterBuilder - SetObjectTypes", "[Chaos][Filter]")
{
	const TArray<uint64> ObjectTypeMasks{ 0, 0xF, 0xAAAAAAAA, 0xFFFFFFFF, FullChannelMask };
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const uint64 ObjectTypeMask = ObjectTypeMasks[Index];

	FQueryObjectFilterBuilder Builder;
	Builder.SetObjectTypes(ObjectTypeMask);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::ObjectType);
	CHECK(QueryFilterData.GetObjectTypesToQueryMask() == ObjectTypeMask);
	CHECK(QueryFilterData.IsMultiQuery() == 0);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryObjectFilterBuilder - SetMultiQuery", "[Chaos][Filter]")
{
	const TArray<bool> bMultiQueryStates{ false, true };
	const uint32 Index = GENERATE(0, 1);
	const bool bMultiQuery = bMultiQueryStates[Index];

	FQueryObjectFilterBuilder Builder;
	Builder.SetMultiQuery(bMultiQuery);
	const FQueryFilterData QueryFilterData = Builder.Build();


	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::ObjectType);
	CHECK(QueryFilterData.GetObjectTypesToQueryMask() == 0);
	CHECK(QueryFilterData.IsMultiQuery() == bMultiQuery);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryObjectFilterBuilder - SetMaskFilter", "[Chaos][Filter]")
{
	const TArray<uint8> MaskFilters{ 0, 1, 2, 0xF };
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const uint8 MaskFilter = MaskFilters[Index];

	FQueryObjectFilterBuilder Builder;
	Builder.SetMaskFilter(MaskFilter);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::ObjectType);
	CHECK(QueryFilterData.GetObjectTypesToQueryMask() == 0);
	CHECK(QueryFilterData.IsMultiQuery() == false);
	CHECK(QueryFilterData.GetIgnoreMask() == MaskFilter);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryObjectFilterBuilder - SetFilterFlags", "[Chaos][Filter]")
{
	const TArray<EFilterFlags> TestFlags
	{
		EFilterFlags::SimpleCollision,
		EFilterFlags::ComplexCollision,
		EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision,
		EFilterFlags::All,
	};
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const EFilterFlags Flags = TestFlags[Index];

	FQueryObjectFilterBuilder Builder;
	Builder.SetFilterFlags(Flags);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::ObjectType);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == 0);
	CHECK(QueryFilterData.GetBlockChannels() == 0);
	CHECK(QueryFilterData.GetOverlapChannels() == 0);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == Flags);
}

TEST_CASE("FQueryObjectFilterBuilder - SetFilterFlags with bool", "[Chaos][Filter]")
{
	FQueryObjectFilterBuilder Builder;
	SECTION("Given Empty Flags: Set Single Flag")
	{
		Builder.SetFilterFlags(EFilterFlags::ComplexCollision, true);
		CHECK(EFilterFlags::ComplexCollision == Builder.Build().GetFlags());
	}
	SECTION("Given Mixed Flags: Set Multiple Flags ")
	{
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, true);
		CHECK((EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision) == Builder.Build().GetFlags());
	}
	SECTION("Given Empty Flags: Set All Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::All, true);
		CHECK(EFilterFlags::All == Builder.Build().GetFlags());
	}
	SECTION("Given All Flags: Clear Single Flag")
	{
		Builder.SetFilterFlags(EFilterFlags::All);
		Builder.SetFilterFlags(EFilterFlags::ComplexCollision, false);
		const EFilterFlags ExpectedFlags = EFilterFlags::All & ~EFilterFlags::ComplexCollision;
		CHECK(ExpectedFlags == Builder.Build().GetFlags());
	}
	SECTION("Given Mixed Flags: Clear Multiple Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, false);
		CHECK((EFilterFlags::None) == Builder.Build().GetFlags());
	}
	SECTION("Given All Flags: Clear All Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::All);
		Builder.SetFilterFlags(EFilterFlags::All, false);
		CHECK(EFilterFlags::None == Builder.Build().GetFlags());
	}
}

TEST_CASE("FQueryObjectFilterBuilder - RoundTrip to legacy and back", "[Chaos][Filter]")
{
	const bool bMultiQuery = GENERATE(false, true);
	FQueryObjectFilterBuilder Builder;
	Builder.SetObjectTypes(0xA);
	Builder.SetMultiQuery(bMultiQuery);
	Builder.SetMaskFilter(0xF);
	Builder.SetFilterFlags(EFilterFlags::All);
	const FQueryFilterData ExpectedFilterData = Builder.Build();

	const FCollisionFilterData TempFilterData = FQueryFilterBuilder::GetLegacyQueryFilter(ExpectedFilterData);
	const FQueryFilterData ResultFilterData = FQueryFilterBuilder::BuildFromLegacyQueryFilter(TempFilterData);

	CHECK(ExpectedFilterData.GetQueryType() == ResultFilterData.GetQueryType());
	CHECK(ExpectedFilterData.GetObjectTypesToQueryMask() == ResultFilterData.GetObjectTypesToQueryMask());
	CHECK(ExpectedFilterData.IsMultiQuery() == ResultFilterData.IsMultiQuery());
	CHECK(ExpectedFilterData.GetIgnoreMask() == ResultFilterData.GetIgnoreMask());
	CHECK(ExpectedFilterData.GetFlags() == ResultFilterData.GetFlags());
}

TEST_CASE("FQueryTraceFilterBuilder - SetCollisionChannelIndex", "[Chaos][Filter]")
{
	const uint8 ChannelIndex = GENERATE(0, 1, 3, 31, 32, 63);

	FQueryTraceFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(ChannelIndex);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::Channel);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == ChannelIndex);
	CHECK(QueryFilterData.GetCollisionChannelMask() == ToChannelMask(ChannelIndex));
	CHECK(QueryFilterData.GetBlockChannels() == 0);
	CHECK(QueryFilterData.GetOverlapChannels() == 0);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryTraceFilterBuilder - SetBlockChannelMask", "[Chaos][Filter]")
{
	const TArray<uint64> BlockChannelMasks{ 0, 1, 2, 3, 0xAAAAAAAA, 0xFFFFFFFF, FullChannelMask };
	const uint32 Index = GENERATE_COPY(Catch::Generators::range(0, BlockChannelMasks.Num()));
	const uint64 BlockChannelMask = BlockChannelMasks[Index];

	FQueryTraceFilterBuilder Builder;
	Builder.SetBlockChannelMask(BlockChannelMask);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::Channel);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == 0);
	CHECK(QueryFilterData.GetBlockChannels() == BlockChannelMask);
	CHECK(QueryFilterData.GetOverlapChannels() == 0);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryTraceFilterBuilder - SetOverlapChannelMask", "[Chaos][Filter]")
{
	const TArray<uint64> OverlapChannelMasks{ 0, 1, 2, 3, 0xAAAAAAAA, 0xFFFFFFFF, FullChannelMask };
	const uint32 Index = GENERATE_COPY(Catch::Generators::range(0, OverlapChannelMasks.Num()));
	const uint64 OverlapChannelMask = OverlapChannelMasks[Index];

	FQueryTraceFilterBuilder Builder;
	Builder.SetOverlapChannelMask(OverlapChannelMask);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::Channel);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == 0);
	CHECK(QueryFilterData.GetBlockChannels() == 0);
	CHECK(QueryFilterData.GetOverlapChannels() == OverlapChannelMask);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryTraceFilterBuilder - SetMaskFilter", "[Chaos][Filter]")
{
	const TArray<uint8> MaskFilters{ 0, 1, 2, 0xF };
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const uint8 MaskFilter = MaskFilters[Index];

	FQueryTraceFilterBuilder Builder;
	Builder.SetMaskFilter(MaskFilter);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::Channel);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == 0);
	CHECK(QueryFilterData.GetBlockChannels() == 0);
	CHECK(QueryFilterData.GetOverlapChannels() == 0);
	CHECK(QueryFilterData.GetIgnoreMask() == MaskFilter);
	CHECK(QueryFilterData.GetFlags() == EmptyFilterFlags);
}

TEST_CASE("FQueryTraceFilterBuilder - SetFilterFlags", "[Chaos][Filter]")
{
	const TArray<EFilterFlags> TestFlags
	{
		EFilterFlags::SimpleCollision,
		EFilterFlags::ComplexCollision,
		EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision,
		EFilterFlags::All,
	};
	const uint32 Index = GENERATE(0, 1, 2, 3);
	const EFilterFlags Flags = TestFlags[Index];

	FQueryTraceFilterBuilder Builder;
	Builder.SetFilterFlags(Flags);
	const FQueryFilterData QueryFilterData = Builder.Build();

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::Channel);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == 0);
	CHECK(QueryFilterData.GetBlockChannels() == 0);
	CHECK(QueryFilterData.GetOverlapChannels() == 0);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == Flags);
}

TEST_CASE("FQueryTraceFilterBuilder - SetFilterFlags with bool", "[Chaos][Filter]")
{
	FQueryTraceFilterBuilder Builder;
	SECTION("Given Empty Flags: Set Single Flag")
	{
		Builder.SetFilterFlags(EFilterFlags::ComplexCollision, true);
		CHECK(EFilterFlags::ComplexCollision == Builder.Build().GetFlags());
	}
	SECTION("Given Mixed Flags: Set Multiple Flags ")
	{
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, true);
		CHECK((EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision) == Builder.Build().GetFlags());
	}
	SECTION("Given Empty Flags: Set All Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::All, true);
		CHECK(EFilterFlags::All == Builder.Build().GetFlags());
	}
	SECTION("Given All Flags: Clear Single Flag")
	{
		Builder.SetFilterFlags(EFilterFlags::All);
		Builder.SetFilterFlags(EFilterFlags::ComplexCollision, false);
		const EFilterFlags ExpectedFlags = EFilterFlags::All & ~EFilterFlags::ComplexCollision;
		CHECK(ExpectedFlags == Builder.Build().GetFlags());
	}
	SECTION("Given Mixed Flags: Clear Multiple Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision);
		Builder.SetFilterFlags(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision, false);
		CHECK((EFilterFlags::None) == Builder.Build().GetFlags());
	}
	SECTION("Given All Flags: Clear All Flags")
	{
		Builder.SetFilterFlags(EFilterFlags::All);
		Builder.SetFilterFlags(EFilterFlags::All, false);
		CHECK(EFilterFlags::None == Builder.Build().GetFlags());
	}
}

TEST_CASE("FQueryTraceFilterBuilder - BuildOverlapAll", "[Chaos][Filter]")
{
	const EFilterFlags Flags = EFilterFlags::ComplexCollision;
	const FQueryFilterData QueryFilterData = FQueryTraceFilterBuilder::BuildOverlapAll(Flags);

	CHECK(QueryFilterData.GetQueryType() == FQueryFilterData::EQueryType::Channel);
	CHECK(QueryFilterData.GetCollisionChannelIndex() == 0);
	CHECK(QueryFilterData.GetBlockChannels() == 0);
	CHECK(QueryFilterData.GetOverlapChannels() == FullChannelMask);
	CHECK(QueryFilterData.GetIgnoreMask() == 0);
	CHECK(QueryFilterData.GetFlags() == Flags);
}

TEST_CASE("FQueryTraceFilterBuilder - RoundTrip to legacy and back", "[Chaos][Filter]")
{
	const uint32 ChannelIndex = GENERATE(0, 1, 31);
	FQueryTraceFilterBuilder Builder;
	Builder.SetCollisionChannelIndex(ChannelIndex);
	Builder.SetBlockChannelMask(2);
	Builder.SetOverlapChannelMask(3);
	Builder.SetMaskFilter(0xF);
	Builder.SetFilterFlags(EFilterFlags::All);
	const FQueryFilterData ExpectedFilterData = Builder.Build();

	const FCollisionFilterData TempFilterData = FQueryFilterBuilder::GetLegacyQueryFilter(ExpectedFilterData);
	const FQueryFilterData ResultFilterData = FQueryFilterBuilder::BuildFromLegacyQueryFilter(TempFilterData);

	CHECK(ExpectedFilterData.GetQueryType() == ResultFilterData.GetQueryType());
	CHECK(ExpectedFilterData.GetCollisionChannelIndex() == ResultFilterData.GetCollisionChannelIndex());
	CHECK(ExpectedFilterData.GetCollisionChannelMask() == ResultFilterData.GetCollisionChannelMask());
	CHECK(ExpectedFilterData.GetBlockChannels() == ResultFilterData.GetBlockChannels());
	CHECK(ExpectedFilterData.GetOverlapChannels() == ResultFilterData.GetOverlapChannels());
	CHECK(ExpectedFilterData.GetIgnoreMask() == ResultFilterData.GetIgnoreMask());
	CHECK(ExpectedFilterData.GetFlags() == ResultFilterData.GetFlags());
}

TEST_CASE("FShapeFilterData - NarrowFilter", "[Chaos][Filter]")
{
	auto BuildFiltersAndRunNarrowFilter = [](const uint8 CollisionChannelIndex0, const uint64 BlockMask0, const uint8 CollisionChannelIndex1, const uint64 BlockMask1)
	{
		const FShapeFilterData Filter0 = FShapeFilterBuilder().SetCollisionChannelIndex(CollisionChannelIndex0).SetBlockChannelMask(BlockMask0).Build();
		const FShapeFilterData Filter1 = FShapeFilterBuilder().SetCollisionChannelIndex(CollisionChannelIndex1).SetBlockChannelMask(BlockMask1).Build();
		return Filter0.NarrowFilter(Filter1);
	};

	// Test the block masks not matching
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b01, 1, 0b01));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b10, 1, 0b10));

	// Test each bit position
	for (uint8 I = 0; I < MaxChannelBitCount; ++I)
	{
		const uint64 ChannelBit = ToChannelMask(I);
		CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(0, ChannelBit, I, 0b01));
		CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(I, 0b01, 0, ChannelBit));
		CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(0, FullChannelMask, I, 0b01));
		CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(I, 0b01, 0, FullChannelMask));
	}

	// Test empty block masks
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b00, 1, 0b00));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b10, 1, 0b00));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b00, 1, 0b01));

	// Test all permutations of 4 bits to get a good sample size
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b1010, 0, 0b0101));
	CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(0, 0b1010, 1, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(0, 0b1010, 2, 0b0101));
	CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(0, 0b1010, 3, 0b0101));

	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(1, 0b1010, 0, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(1, 0b1010, 1, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(1, 0b1010, 2, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(1, 0b1010, 3, 0b0101));

	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(2, 0b1010, 0, 0b0101));
	CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(2, 0b1010, 1, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(2, 0b1010, 2, 0b0101));
	CHECK(ENarrowFilterResult::Block == BuildFiltersAndRunNarrowFilter(2, 0b1010, 3, 0b0101));

	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(3, 0b1010, 0, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(3, 0b1010, 1, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(3, 0b1010, 2, 0b0101));
	CHECK(ENarrowFilterResult::None == BuildFiltersAndRunNarrowFilter(3, 0b1010, 3, 0b0101));
}

TEST_CASE("FQueryFilterData - NarrowFilter - ObjectType", "[Chaos][Filter]")
{
	auto BuildAndRunObjectTypeNarrowFilter = [](const uint8 ShapeCollisionChannel, const uint64 ObjectTypeMask)
	{
		const FShapeFilterData ShapeFilter = FShapeFilterBuilder().SetCollisionChannelIndex(ShapeCollisionChannel).Build();
		const FQueryFilterData QueryFilter = FQueryObjectFilterBuilder().SetObjectTypes(ObjectTypeMask).Build();
		return QueryFilter.NarrowFilter(ShapeFilter);
	};
	auto BuildAndRunObjectTypeNarrowFilterWithMasks = [](const uint8 CollisionChannel, const uint8 ShapeMaskFilter, const uint8 QueryMaskFilter)
	{
		const FShapeFilterData ShapeFilter = FShapeFilterBuilder().SetCollisionChannelIndex(CollisionChannel).SetMaskFilter(ShapeMaskFilter).Build();
		const FQueryFilterData QueryFilter = FQueryObjectFilterBuilder().SetObjectTypes(ToChannelMask(CollisionChannel)).SetMaskFilter(QueryMaskFilter).Build();
		return QueryFilter.NarrowFilter(ShapeFilter);

	};
	for (uint8 I = 0; I < MaxChannelBitCount; ++I)
	{
		CHECK(ENarrowFilterResult::None == BuildAndRunObjectTypeNarrowFilter(I, 0));
		CHECK(ENarrowFilterResult::Block == BuildAndRunObjectTypeNarrowFilter(I, ToChannelMask(I)));
		CHECK(ENarrowFilterResult::Block == BuildAndRunObjectTypeNarrowFilter(I, FullChannelMask));
	}

	CHECK(ENarrowFilterResult::None == BuildAndRunObjectTypeNarrowFilter(0, 0b1010));
	CHECK(ENarrowFilterResult::Block == BuildAndRunObjectTypeNarrowFilter(1, 0b1010));
	CHECK(ENarrowFilterResult::None == BuildAndRunObjectTypeNarrowFilter(2, 0b1010));
	CHECK(ENarrowFilterResult::Block == BuildAndRunObjectTypeNarrowFilter(3, 0b1010));
	CHECK(ENarrowFilterResult::None == BuildAndRunObjectTypeNarrowFilter(4, 0b1010));

	// Verify the mask filter
	CHECK(ENarrowFilterResult::Block == BuildAndRunObjectTypeNarrowFilterWithMasks(3, 0b0110, 0b1001));
	CHECK(ENarrowFilterResult::None == BuildAndRunObjectTypeNarrowFilterWithMasks(3, 0b0110, 0b1100));
}

TEST_CASE("FQueryFilterData - NarrowFilter - Channel", "[Chaos][Filter]")
{
	auto BuildAndRunChannelNarrowFilter = [](const uint8 ShapeChannel, const uint64 ShapeOverlap, const uint64 ShapeBlock, const uint8 QueryChannel, const uint64 QueryOverlap, const uint64 QueryBlock)
	{
		const FShapeFilterData ShapeFilter = FShapeFilterBuilder()
			.SetCollisionChannelIndex(ShapeChannel)
			.SetOverlapChannelMask(ShapeOverlap)
			.SetBlockChannelMask(ShapeBlock)
			.Build();
		const FQueryFilterData QueryFilter = FQueryTraceFilterBuilder()
			.SetCollisionChannelIndex(QueryChannel)
			.SetOverlapChannelMask(QueryOverlap)
			.SetBlockChannelMask(QueryBlock)
			.Build();
		return QueryFilter.NarrowFilter(ShapeFilter);
	};
	auto BuildAndRunChannelTypeNarrowFilterWithMasks = [](const uint8 ChannelIndex, const uint8 ShapeMaskFilter, const uint8 QueryMaskFilter)
	{
		const uint64 BlockMask = ToChannelMask(ChannelIndex);
		const FShapeFilterData ShapeFilter = FShapeFilterBuilder()
			.SetCollisionChannelIndex(ChannelIndex)
			.SetBlockChannelMask(BlockMask)
			.SetMaskFilter(ShapeMaskFilter)
			.Build();
		const FQueryFilterData QueryFilter = FQueryTraceFilterBuilder()
			.SetCollisionChannelIndex(ChannelIndex)
			.SetBlockChannelMask(BlockMask)
			.SetMaskFilter(QueryMaskFilter)
			.Build();
		return QueryFilter.NarrowFilter(ShapeFilter);
	};

	// Rules: 
	// - For a single check: Block > Overlap > None (e.g. Block & Overlap -> Block)
	// - For a combined check: None > Overlap > Block (e.g. Block vs. Overlap -> Overlap)

	constexpr uint64 FullMask = FullChannelMask;
	for (uint8 I = 0; I < MaxChannelBitCount; ++I)
	{
		const uint64 ChannelBit = ToChannelMask(I);
		// (None, None) -> None
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, 0, 0));
		// (None, Overlap) -> None
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, ChannelBit, 0));
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, FullMask, 0));
		// (None, Block) -> None
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, 0, ChannelBit));
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, 0, FullMask));
		// (None, Overlap & Block) -> None
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, ChannelBit, ChannelBit));
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, 0, I, FullMask, FullMask));

		// (Overlap, None) -> None
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, ChannelBit, 0, I, 0, 0));
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, FullMask, 0, I, 0, 0));
		// (Overlap, Overlap) -> Overlap
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, ChannelBit, 0, I, ChannelBit, 0));
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, FullMask, 0, I, FullMask, 0));
		// (Overlap, Block) -> Overlap
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, ChannelBit, 0, I, 0, ChannelBit));
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, FullMask, 0, I, 0, FullMask));
		// (Overlap, Overlap & Block) -> Overlap
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, ChannelBit, 0, I, ChannelBit, ChannelBit));
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, FullMask, 0, I, FullMask, FullMask));

		// (Block, None) -> None
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, ChannelBit, I, 0, 0));
		CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(I, 0, FullMask, I, 0, 0));
		// (Block, Overlap) -> Overlap
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, 0, ChannelBit, I, ChannelBit, 0));
		CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(I, 0, FullMask, I, FullMask, 0));
		// (Block, Block) -> Block
		CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(I, 0, ChannelBit, I, 0, ChannelBit));
		CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(I, 0, FullMask, I, 0, FullMask));
		// (Block, Overlap & Block) -> Block
		CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(I, 0, ChannelBit, I, ChannelBit, ChannelBit));// Broken??
		CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(I, 0, FullMask, I, FullMask, FullMask));

		// (Overlap & Block, Overlap & Block) -> Block
		CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(I, ChannelBit, ChannelBit, I, ChannelBit, ChannelBit));
		CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(I, FullMask, FullMask, I, FullMask, FullMask));
	}

	// Test individual bit patterns...

	// Block vs.
	CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(0, 0b10100000, 0b01010000, 4, 0b00001010, 0b00000101));
	CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(1, 0b10100000, 0b01010000, 4, 0b00001010, 0b00000101));
	// Overlap vs.
	CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(2, 0b10100000, 0b01010000, 5, 0b00001010, 0b00000101));
	CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(3, 0b10100000, 0b01010000, 5, 0b00001010, 0b00000101));
	// Block vs.
	CHECK(ENarrowFilterResult::Block == BuildAndRunChannelNarrowFilter(0, 0b10100000, 0b01010000, 6, 0b00001010, 0b00000101));
	CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(1, 0b10100000, 0b01010000, 6, 0b00001010, 0b00000101));
	// Overlap vs.
	CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(2, 0b10100000, 0b01010000, 7, 0b00001010, 0b00000101));
	CHECK(ENarrowFilterResult::Overlap == BuildAndRunChannelNarrowFilter(3, 0b10100000, 0b01010000, 7, 0b00001010, 0b00000101));
	// Test edge boundaries for none
	CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(0, 0b10100000, 0b01010000, 3, 0b00001010, 0b00000101));
	CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(0, 0b10100000, 0b01010000, 8, 0b00001010, 0b00000101));
	CHECK(ENarrowFilterResult::None == BuildAndRunChannelNarrowFilter(5, 0b10100000, 0b01010000, 4, 0b00001010, 0b00000101));

	// Verify the mask filter
	CHECK(ENarrowFilterResult::Block == BuildAndRunChannelTypeNarrowFilterWithMasks(3, 0b0110, 0b1001));
	CHECK(ENarrowFilterResult::None == BuildAndRunChannelTypeNarrowFilterWithMasks(3, 0b0110, 0b1100));
}

TEST_CASE("FShapeUnionFilterData - BroadFilter - ShapeUnion", "[Chaos][Filter]")
{
	// Single
	SECTION("Single - Accept")
	{
		const FShapeUnionFilterData Union0 = BuildUnionFilter({ BuildShapeFilter(0, 0b10, 0) });
		const FShapeUnionFilterData Union1 = BuildUnionFilter({ BuildShapeFilter(1, 0b01, 0) });
		CHECK(EBroadFilterResult::Accept == Union0.BroadFilter(Union1));
	}
	SECTION("Single - Reject")
	{
		const FShapeUnionFilterData Union0 = BuildUnionFilter({ BuildShapeFilter(0, 0b10, 0) });
		const FShapeUnionFilterData Union1 = BuildUnionFilter({ BuildShapeFilter(1, 0b10, 0) });
		CHECK(EBroadFilterResult::Reject == Union0.BroadFilter(Union1));
	}
	SECTION("Multiple - Both Reject")
	{
		// A { Channel(0b0101) Block(0b1010) } B { Channel(0b0101) Block(0b1010) }
		const FShapeUnionFilterData Union0 = BuildUnionFilter({ BuildShapeFilter(0, 0b0010, 0), BuildShapeFilter(2, 0b1000, 0) });
		const FShapeUnionFilterData Union1 = BuildUnionFilter({ BuildShapeFilter(0, 0b0010, 0), BuildShapeFilter(2, 0b1000, 0) });
		CHECK(EBroadFilterResult::Reject == Union0.BroadFilter(Union1));
	}
	SECTION("Multiple - One Rejects")
	{
		// A { Channel(0b0101) Block(0b1010) } B { Channel(0b0101) Block(0b0011) }
		const FShapeUnionFilterData Union0 = BuildUnionFilter({ BuildShapeFilter(0, 0b0010, 0), BuildShapeFilter(2, 0b1000, 0) });
		const FShapeUnionFilterData Union1 = BuildUnionFilter({ BuildShapeFilter(0, 0b0010, 0), BuildShapeFilter(2, 0b0001, 0) });
		CHECK(EBroadFilterResult::Reject == Union0.BroadFilter(Union1));
		CHECK(EBroadFilterResult::Reject == Union1.BroadFilter(Union0));
	}
	SECTION("Multiple - Accept")
	{
		// A { Channel(0b0011) Block(0b1100) } B { Channel(0b1100) Block(0b0011) }
		const FShapeUnionFilterData Union0 = BuildUnionFilter({ BuildShapeFilter(0, 0b1000, 0), BuildShapeFilter(1, 0b0100, 0) });
		const FShapeUnionFilterData Union1 = BuildUnionFilter({ BuildShapeFilter(2, 0b0010, 0), BuildShapeFilter(3, 0b0001, 0) });
		CHECK(EBroadFilterResult::Accept == Union0.BroadFilter(Union1));
	}
	SECTION("Combine No Sim")
	{
		const FShapeFilterData ShapeFilter0 = BuildShapeFilter(0, 0b01, 0);
		const FShapeFilterData ShapeFilter1 = BuildShapeFilter(1, 0b10, 0);

		const FShapeUnionFilterData Union0 = BuildUnionFilter({ ShapeFilter0 });
		const FShapeUnionFilterData Union1 = BuildUnionFilter({ ShapeFilter1 });
		FShapeUnionFilterData Union01;
		Union01.Combine(ShapeFilter0, true, true);
		Union01.Combine(ShapeFilter1, true, false);

		CHECK(EBroadFilterResult::Accept == Union01.BroadFilter(Union0));
		CHECK(EBroadFilterResult::Reject == Union01.BroadFilter(Union1));
	}
}

TEST_CASE("FShapeUnionFilterData - BroadFilter - Query", "[Chaos][Filter]")
{
	const uint8 ChannelIndex = 0;

	SECTION("QueryType(Channel)")
	{
		const TArray<FQueryFilterData> QueryFilters
		{
			FQueryTraceFilterBuilder().SetCollisionChannelIndex(0).Build(),
			FQueryTraceFilterBuilder().SetCollisionChannelIndex(1).Build(),
			FQueryTraceFilterBuilder().SetCollisionChannelIndex(2).Build(),
			FQueryTraceFilterBuilder().SetCollisionChannelIndex(3).Build(),
		};
		SECTION("Empty Filters")
		{
			const FQueryFilterData QueryFilter = FQueryTraceFilterBuilder().Build();
			FShapeUnionFilterData Union0;
			CHECK(EBroadFilterResult::Reject == Union0.BroadFilter(QueryFilter));

			FShapeUnionFilterData Union1;
			Union1.Combine(FShapeFilterBuilder().Build());
			CHECK(EBroadFilterResult::Reject == Union1.BroadFilter(QueryFilter));
		}
		SECTION("Block Masks")
		{
			FShapeUnionFilterData Union;
			Union.Combine(BuildShapeFilter(ChannelIndex, 0b0001, 0));
			Union.Combine(BuildShapeFilter(ChannelIndex, 0b0100, 0));

			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[0]));
			CHECK(EBroadFilterResult::Reject == Union.BroadFilter(QueryFilters[1]));
			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[2]));
			CHECK(EBroadFilterResult::Reject == Union.BroadFilter(QueryFilters[3]));
		}
		SECTION("Overlap Masks")
		{
			FShapeUnionFilterData Union;
			Union.Combine(BuildShapeFilter(ChannelIndex, 0, 0b0001));
			Union.Combine(BuildShapeFilter(ChannelIndex, 0, 0b0100));

			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[0]));
			CHECK(EBroadFilterResult::Reject == Union.BroadFilter(QueryFilters[1]));
			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[2]));
			CHECK(EBroadFilterResult::Reject == Union.BroadFilter(QueryFilters[3]));
		}
		SECTION("Block and Overlap Masks")
		{
			FShapeUnionFilterData Union;
			// Note: Bit 2 is true for both
			Union.Combine(BuildShapeFilter(ChannelIndex, 0b0001, 0b0000));
			Union.Combine(BuildShapeFilter(ChannelIndex, 0b0010, 0b0000));
			Union.Combine(BuildShapeFilter(ChannelIndex, 0b0000, 0b0010));
			Union.Combine(BuildShapeFilter(ChannelIndex, 0b0000, 0b1000));

			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[0]));
			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[1]));
			CHECK(EBroadFilterResult::Reject == Union.BroadFilter(QueryFilters[2]));
			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[3]));
		}
		SECTION("Combine No Sim")
		{
			FShapeUnionFilterData Union;
			Union.Combine(BuildShapeFilter(ChannelIndex, 0, 0b01));
			Union.Combine(BuildShapeFilter(ChannelIndex, 1, 0b10), false);

			CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilters[0]));
			CHECK(EBroadFilterResult::Reject == Union.BroadFilter(QueryFilters[1]));
		}
	}

	SECTION("QueryType(ObjectType)")
	{
		// Note: ObjectType filters do no filtering now
		const FQueryFilterData QueryFilter0 = FQueryObjectFilterBuilder().Build();
		const FQueryFilterData QueryFilter1 = FQueryObjectFilterBuilder().SetObjectTypes(0b01).Build();
		const FQueryFilterData QueryFilter2 = FQueryObjectFilterBuilder().SetObjectTypes(0b10).Build();

		FShapeUnionFilterData Union;
		Union.Combine(BuildShapeFilter(ChannelIndex, 0b0001, 0b0001));

		CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilter0));
		CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilter1));
		CHECK(EBroadFilterResult::Accept == Union.BroadFilter(QueryFilter2));
	}
}

} // namespace Chaos::Filter
