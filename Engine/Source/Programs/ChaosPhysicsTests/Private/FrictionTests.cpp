// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "Chaos/Box.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ContactModification.h"
#include "Chaos/MassProperties.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/SimCallbackObject.h"
#include "ChaosSolversModule.h"
#include "ChaosTestHarness.h"
#include "ChaosTestScene.h"
#include "EventsData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "ScopedCVar.h"

//
// Test initial friction by dropping a moving ball onto a plane with enough 
// friction to stop the contact point.
// The results should be independent of initial penetration depth.
// The results should be independent of friction as long as it is above the sticking limit.
//
TEST_CASE("ChaosImpactFrictionTests", "[Chaos][Friction][unit]")
{
	using namespace Chaos;
	using namespace Chaos::LowLevelTest;

	const float Dt = 1.0f / 30.0f;
	const float G = 1000.0f;

	FChaosTestScene Scene;
	Scene.SetGravity(FVector3d(0,0,-G));

	// Place with initial separation as a fraction of the distance moved 
	// under gravity in one tick when starting from rest.
	// These leads to penetration depths of: (1.11, 0.83, 0.22, 0.01)
	float SeparationAlpha = GENERATE(0.0f, 0.25f, 0.75f, 0.99f);
	float Separation = SeparationAlpha * G * Dt * Dt;

	SECTION("Moving ball with high friction")
	{
		const double KinematicPositionTolerance = UE_SMALL_NUMBER;
		const double DynamicPositionTolerance = UE_KINDA_SMALL_NUMBER;

		// Try different friction values (all should be above static friction threshold)
		auto Friction = GENERATE(1000.0f, 10.0f, 1.0f);

		// Place a dynamic 50cm radius sphere exactly on the floor (a large kinematic box)
		FSingleParticlePhysicsProxy* FloorBox = Scene.AddKinematicBox(TEXT("Floor"), FTransform3d(FVector3d(0, 0, -50)), FVector3d(500, 500, 50));
		FSingleParticlePhysicsProxy* DynamicSphere = Scene.AddDynamicSphere(TEXT("Sphere"), FTransform3d(FVector3d(0, 0, 50 + Separation)), 50);

		// Add horizontal velocity
		DynamicSphere->GetGameThreadAPI().SetV(FVector3d(100, 0, 0));

		// Set the friction to desired value in a callback
		// Easier than setting up materials
		FChaosTestSimCallback* SimCallbacks = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FChaosTestSimCallback>();
		SimCallbacks->ContactModifier = 
			[DynamicSphere, Friction](FCollisionContactModifier& Contacts)
			{
				for (FContactPairModifier& Contact : Contacts.GetContacts(DynamicSphere->GetHandle_LowLevel()))
				{
					Contact.ModifyStaticFriction(Friction);
					Contact.ModifyDynamicFriction(Friction);
				}
			};

		Scene.TickSolver(Dt);

		// Contact point velocity should be zero ish
		FVector3d V = DynamicSphere->GetGameThreadAPI().GetV();
		FVector3d W = DynamicSphere->GetGameThreadAPI().GetW();
		FVector3d VC = V + FVector3d::CrossProduct(W, FVector3d(0,0,-50));

		REQUIRE(Scene.GetSolver()->GetEvolution()->GetCollisionConstraints().NumConstraints() == 1);
		const FPBDCollisionConstraint& Collision = Scene.GetSolver()->GetEvolution()->GetCollisionConstraints().GetConstConstraint(0);
		REQUIRE(Collision.NumManifoldPoints() == 1);
		const FManifoldPoint& ManifoldPoint = Collision.GetManifoldPoint(0);
		float Phi = ManifoldPoint.ContactPoint.Phi;

		CAPTURE(G, Dt, Friction, Phi, V.X, VC.X);

		CHECK_THAT(VC.Size(), Catch::Matchers::WithinAbs(0.0f, 0.2f));
	}
}
