// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "Chaos/Box.h"
#include "Chaos/MassConditioning.h"
#include "Chaos/MassProperties.h"
#include "ChaosTestHarness.h"

TEST_CASE("ChaosMassConditioningTests", "[Chaos][MassConditioning][unit]")
{
	using namespace Chaos;

	FRealSingle MaxDistance = 10.0f;
	FRealSingle MaxRotationRatio = 2.0f;
	FRealSingle MaxInvInertiaComponentRatio = 0.0f;

	// Create a box and pretend we have a constraint a long way from the box.
	// Inertia conditioning will increase the inertia of the box so that impulses
	// applied at the constraint handle do not introduce crazy rotation.
	// BUG: Previously this would fail to condition inertia for large objects
	// because of a tolerance check on the inverse mass and inertia which can 
	// legitimately be very small.
	// REBUG: I had to undo the previous fix because it changed behaviour in
	// a way that can affect existing projects. E.g., vehicles will roll more
	// easily if InertiaConditioning now affects them but didn't before.
	SECTION("Small inverse inertia")
	{
		FRealSingle BoxExtent = GENERATE(10000.0f, 1000.0f, 100.0f, 10.0f);
		FRealSingle ConstraintArmMultiplier = GENERATE(1.5f, 5.0f, 10.0f, 100.0f);

		FImplicitBox3 Box(FVec3(-BoxExtent), FVec3(BoxExtent), 0);

		FReal DensityKGPerCM3 = 1.e-3f;
		FMassProperties MassProperties;
		CalculateMassPropertiesOfImplicitType(MassProperties, FTransform::Identity, &Box, DensityKGPerCM3);
		TransformToLocalSpace(MassProperties);

		FRealSingle InvM = 1.0f / MassProperties.Mass;
		FVec3f InvI = FVec3f(1.0f) / MassProperties.InertiaTensor.GetDiagonal();

		// Pretend we have a constraint with an anchor along the X axis outside the box by some distance.
		FVec3f ConstraintExtents = FVec3f(ConstraintArmMultiplier * BoxExtent, BoxExtent, BoxExtent);

		// Default Tolerances
		FInertiaConditioningTolerances Tolerances;
		CHECK(Tolerances.InvInertiaTolerance == 1.e-4f);	// Tested with this value

		// If the inputs pass the tolerances, Inertia conditioning will increase the Y and Z inertia (i.e., reduce inverse inertia)
		// so that any rotations applied by the constraint anchor are "small" (based on the settings)
		FVec3f InertiaConditioning = CalculateInertiaConditioning(InvM, InvI, ConstraintExtents, MaxDistance, MaxRotationRatio, MaxInvInertiaComponentRatio, Tolerances);

		bool bIsSmallInvInertia = InvI.GetMin() < Tolerances.InvInertiaTolerance;
		if (bIsSmallInvInertia)
		{
			CHECK_THAT(InertiaConditioning.X, Catch::Matchers::WithinAbs(1.0f, 0.01f));
			CHECK_THAT(InertiaConditioning.Y, Catch::Matchers::WithinAbs(1.0f, 0.01f));
			CHECK_THAT(InertiaConditioning.Z, Catch::Matchers::WithinAbs(1.0f, 0.01f));
		}
		else
		{
			// TODO: These tests should really verify that applying an impulse at the constaint arm results in a
			// correction that has at most MaxRotationRatio of its magnitude from rotation
			CHECK_THAT(InertiaConditioning.X, Catch::Matchers::WithinAbs(1.0f, 0.01f));
			CHECK(InertiaConditioning.Y < 1.0f);
			CHECK(InertiaConditioning.Z < 1.0f);
		}
	}
}