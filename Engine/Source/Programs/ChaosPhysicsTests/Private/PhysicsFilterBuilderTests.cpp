// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"

#include "ChaosTestHarness.h"

#include "Physics/PhysicsFiltering.h"

namespace Chaos::Tests
{
	using FShapeFilterData = Chaos::Filter::FShapeFilterData;
	constexpr uint64 ChannelCount = 64;
	constexpr uint64 FullChannelMask = std::numeric_limits<uint64>::max();

	TEST_CASE("PhysicsFilterBuilder - SetResponses", "[Chaos][Filter]")
	{
		SECTION("Test each block bit")
		{
			for (uint64 I = 0; I < ChannelCount; ++I)
			{
				FCollisionResponseContainer Container(ECR_Ignore);
				Container.EnumArray[I] = ECR_Block;

				FPhysicsFilterBuilder Builder;
				Builder.SetResponses(Container);
				const FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();

				CHECK((1LLU << I) == ShapeFilter.GetBlockChannels());
				CHECK(0 == ShapeFilter.GetOverlapChannels());
			}
		}
		SECTION("Test each overlap bit")
		{
			for (uint64 I = 0; I < ChannelCount; ++I)
			{
				FCollisionResponseContainer Container(ECR_Ignore);
				Container.EnumArray[I] = ECR_Overlap;

				FPhysicsFilterBuilder Builder;
				Builder.SetResponses(Container);
				const FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();

				CHECK(0 == ShapeFilter.GetBlockChannels());
				CHECK((1LLU << I) == ShapeFilter.GetOverlapChannels());
			}
		}
		SECTION("Test full ignore mask")
		{
			FCollisionResponseContainer Container(ECR_Ignore);

			FPhysicsFilterBuilder Builder;
			Builder.SetResponses(Container);
			const FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();

			CHECK(0 == ShapeFilter.GetBlockChannels());
			CHECK(0 == ShapeFilter.GetOverlapChannels());
		}
		SECTION("Test full block mask")
		{
			FCollisionResponseContainer Container(ECR_Block);

			FPhysicsFilterBuilder Builder;
			Builder.SetResponses(Container);
			const Chaos::Filter::FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();

			CHECK(FullChannelMask == ShapeFilter.GetBlockChannels());
			CHECK(0 == ShapeFilter.GetOverlapChannels());
		}
		SECTION("Test full overlap mask")
		{
			FCollisionResponseContainer Container(ECR_Overlap);

			FPhysicsFilterBuilder Builder;
			Builder.SetResponses(Container);
			const FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();

			CHECK(0 == ShapeFilter.GetBlockChannels());
			CHECK(FullChannelMask == ShapeFilter.GetOverlapChannels());
		}
		SECTION("Test mixed masks")
		{
			FCollisionResponseContainer Container;
			for (uint64 I = 0; I < ChannelCount; ++I)
			{
				Container.EnumArray[I] = I % 3;
			}

			FPhysicsFilterBuilder Builder;
			Builder.SetResponses(Container);
			const FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();

			CHECK(0x4924924924924924 == ShapeFilter.GetBlockChannels());
			CHECK(0x2492492492492492 == ShapeFilter.GetOverlapChannels());
		}
	}

	TEST_CASE("PhysicsFilterBuilder - SetResponses - perf", "[Chaos][Filter][!benchmark]")
	{
		constexpr uint64 Iterations = 100000;

		FCollisionResponseContainer Container;
		for (uint64 I = 0; I < ChannelCount; ++I)
		{
			Container.EnumArray[I] = I % 3;
		}

		BENCHMARK("Set Responses")
		{
			uint64 BlockResult = 0;
			uint64 OverlapResult = 0;
			for (uint64 I = 0; I < Iterations; ++I)
			{
				FPhysicsFilterBuilder Builder(nullptr);
				Builder.SetResponses(Container);
				const FShapeFilterData ShapeFilter = Builder.BuildShapeFilterData();
				BlockResult |= ShapeFilter.GetBlockChannels();
				OverlapResult |= ShapeFilter.GetOverlapChannels();
			}
			return BlockResult | OverlapResult;
		};
	}
} // namespace Chaos::Tests
