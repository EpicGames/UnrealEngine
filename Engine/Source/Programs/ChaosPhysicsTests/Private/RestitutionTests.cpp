// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "Chaos/Box.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ContactModification.h"
#include "Chaos/MassProperties.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/SimCallbackObject.h"
#include "ChaosSolversModule.h"
#include "ChaosTestHarness.h"
#include "ChaosTestScene.h"
#include "EventsData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "ScopedCVar.h"

//
// Test that a dropped box bounces correctly
//
TEST_CASE("ChaosRestitutionTests", "[Chaos][Restitution][unit]")
{
	using namespace Chaos;
	using namespace Chaos::LowLevelTest;

	const float Dt = 1.0f / 60.0f;

	FChaosTestScene Scene;
	Scene.SetGravity(FVector3d(0));

	// 4 iterations required for the thresholds we set
	int32 NumVelIts = GENERATE(8, 6, 4);
	Scene.GetSolver()->SetVelocityIterations(NumVelIts);

	// Try different friction values
	float Restitution = 1.0f;
	float Friction = GENERATE(0.0f, 1.0f);

	// Place just above the ground with high velocity
	float DropHeight = 1.0f;
	float InitialVelZ = -400.0f;

	SECTION("A box dropped onto flat ground should bounce straight back")
	{
		// Place a dynamic box above the floor
		FSingleParticlePhysicsProxy* FloorBox = Scene.AddKinematicBox(TEXT("Floor"), FTransform3d(FVector3d(0, 0, -50)), FVector3d(500, 500, 50));
		FSingleParticlePhysicsProxy* DynamicSphere = Scene.AddDynamicBox(TEXT("Box"), FTransform3d(FVector3d(0, 0, 50 + DropHeight)), FVector3d(50, 50, 50));

		DynamicSphere->GetGameThreadAPI().SetV(FVector3d(0, 0, InitialVelZ));

		// Set the friction and restitution to desired value in a callback
		// Easier than setting up materials
		FChaosTestSimCallback* SimCallbacks = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FChaosTestSimCallback>();
		SimCallbacks->ContactModifier = 
			[DynamicSphere, Friction, Restitution](FCollisionContactModifier& Contacts)
			{
				for (FContactPairModifier& Contact : Contacts.GetContacts(DynamicSphere->GetHandle_LowLevel()))
				{
					Contact.ModifyStaticFriction(Friction);
					Contact.ModifyDynamicFriction(Friction);
					Contact.ModifyRestitution(Restitution);
				}
			};

		FVector3d PreV = DynamicSphere->GetGameThreadAPI().GetV();
		FVector3d PreW = DynamicSphere->GetGameThreadAPI().GetW();

		Scene.TickSolver(Dt);

		FVector3d PostV = DynamicSphere->GetGameThreadAPI().GetV();
		FVector3d PostW = DynamicSphere->GetGameThreadAPI().GetW();

		const FPBDCollisionConstraints& CollisionConstraints = Scene.GetSolver()->GetEvolution()->GetCollisionConstraints();
		REQUIRE(CollisionConstraints.NumConstraints() == 1);

		const FPBDCollisionConstraint& Collision =CollisionConstraints.GetConstConstraint(0);
		REQUIRE(Collision.NumManifoldPoints() == 4);

		const FManifoldPoint& ManifoldPoint = Collision.GetManifoldPoint(0);
		float Phi = ManifoldPoint.ContactPoint.Phi;

		CAPTURE(NumVelIts, Friction, Restitution, InitialVelZ, Phi, PreV.Z, PostV.Z, PreW.Size(), PostW.Size());
		CHECK_THAT(PostV.Z, Catch::Matchers::WithinAbs(-Restitution * PreV.Z, 1.0f));
		CHECK_THAT(PostW.Size(), Catch::Matchers::WithinAbs(0.0f, 0.01f));
	}
}
