// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/IConsoleManager.h"

template<typename T>
class FCVarSetter
{
public:
	FCVarSetter(const FString& InConsoleVariableName)
		: ConsoleVariableName(InConsoleVariableName)
		, OriginalValue()
		, bIsSet(false)
	{
	}

	virtual ~FCVarSetter()
	{
	}

	void Set(T Value);

	void Restore()
	{
		if (bIsSet)
		{
			if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName))
			{
				ConsoleVariable->AsVariable()->SetWithCurrentPriority(OriginalValue);
			}
			bIsSet = false;
		}
	}

private:
	FString ConsoleVariableName;
	T OriginalValue;
	bool bIsSet;
};

template<>
FORCEINLINE void FCVarSetter<bool>::Set(bool Value)
{
	check(!bIsSet);
	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName))
	{
		OriginalValue = ConsoleVariable->GetBool();
		ConsoleVariable->AsVariable()->SetWithCurrentPriority(Value);
		bIsSet = true;
	}
}

template<>
FORCEINLINE void FCVarSetter<int32>::Set(int32 Value)
{
	check(!bIsSet);
	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName))
	{
		OriginalValue = ConsoleVariable->GetInt();
		ConsoleVariable->AsVariable()->SetWithCurrentPriority(Value);
		bIsSet = true;
	}
}

template<>
FORCEINLINE void FCVarSetter<float>::Set(float Value)
{
	check(!bIsSet);
	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName))
	{
		OriginalValue = ConsoleVariable->GetFloat();
		ConsoleVariable->AsVariable()->SetWithCurrentPriority(Value);
		bIsSet = true;
	}
}

template<typename T>
class FCVarScopeGuard
{
public:
	FCVarScopeGuard(const FString& InConsoleVariableName, T InValue) 
		: CVarSetter(InConsoleVariableName)
	{
		CVarSetter.Set(InValue);
	}

	~FCVarScopeGuard()
	{
		CVarSetter.Restore();
	}
private:
	FCVarSetter<T> CVarSetter;
};
