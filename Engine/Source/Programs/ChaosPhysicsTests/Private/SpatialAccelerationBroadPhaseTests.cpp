// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"
#include "ChaosTestScene.h"
#include "ChaosSolversModule.h"

#include "Chaos/Box.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace Chaos
{
	TEST_CASE("SpatialAccelerationBroadPhase - PrePreFilter", "[Chaos][BroadPhase][unit]")
	{
		using namespace Chaos::LowLevelTest;

		// This test is for a bug where the full collision filter data was or'd together, including word 3's collision channel index, which would do invalid filtering with union shapes.
		constexpr float Dt = 1.0f / 30.0f;
		FChaosTestScene Scene;

		const FVector3d HalfExtents(50, 50, 50);
		TArray<FImplicitObjectPtr> Geometry
		{
			FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents)),
			FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents)),
		};
		FImplicitObjectPtr UnionImplicit = FImplicitObjectPtr(new FImplicitObjectUnion(MoveTemp(Geometry)));
		FSingleParticlePhysicsProxy* UnionProxy0 = Scene.AddKinematicProxy(TEXT("Union0"), FTransform3d(FVector3d(0, 0, 0)), UnionImplicit);
		FSingleParticlePhysicsProxy* UnionProxy1 = Scene.AddDynamicProxy(TEXT("Union1"), FTransform3d(FVector3d(0, 0, 60)), UnionImplicit);

		Private::FCollisionConstraintAllocator CollisionAllocator;
		CollisionAllocator.SetMaxContexts(1);
		FCollisionDetectorSettings Settings;
		Chaos::FSpatialAccelerationBroadPhase& BroadPhase = Scene.GetSolver()->GetEvolution()->GetBroadPhase();

		auto SetShapeFilter = [](FSingleParticlePhysicsProxy* Proxy, const int32 ShapeIndex, uint8 CollisionChannelIndex, Chaos::EFilterFlags Flags, uint32 BlockMask)
		{
			Filter::FShapeFilterBuilder Builder;
			Builder.SetCollisionChannelIndex(CollisionChannelIndex);
			Builder.SetBlockChannelMask(BlockMask);
			Builder.SetFilterFlags(Flags);
			const Filter::FShapeFilterData FilterData = Builder.Build();

			Proxy->GetGameThreadAPI().SetShapeFilterData(ShapeIndex, FilterData);
		};

		// Test cases that should produce no overlap pairs.
		{
			// Test everything blocking channel 3. Proxy0 is set to channels 1 and 2 which if incorrectly 
			// bitwise or'd together would turn into channel 3. This should not return any pairs.
			SetShapeFilter(UnionProxy0, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 0, 3, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 1, 3, EFilterFlags::All, 0b01000);

			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 0);

			// Test the opposite order to ensure ordering doesn't matter.
			SetShapeFilter(UnionProxy0, 0, 3, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 1, 3, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 1, 2, EFilterFlags::All, 0b01000);

			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 0);
		}

		// Now test cases that should produce an overlap.
		{
			// Test where Shape10 should overlap Shape01.
			SetShapeFilter(UnionProxy0, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 0, 3, EFilterFlags::All, 0b00100);
			SetShapeFilter(UnionProxy1, 1, 3, EFilterFlags::All, 0b01000);

			// Test where Shape10 should overlap Shape00.
			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 1);

			SetShapeFilter(UnionProxy0, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 0, 3, EFilterFlags::All, 0b00010);
			SetShapeFilter(UnionProxy1, 1, 3, EFilterFlags::All, 0b01000);
			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 1);

			// Test where Shape11 should overlap both shapes on proxy0.
			SetShapeFilter(UnionProxy0, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 0, 3, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 1, 3, EFilterFlags::All, 0b00110);
			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 1);
		}

		// Test the opposite ordering to be safe.
		{
			// Test where Shape00 should overlap Shape11.
			SetShapeFilter(UnionProxy1, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 0, 3, EFilterFlags::All, 0b00100);
			SetShapeFilter(UnionProxy0, 1, 3, EFilterFlags::All, 0b01000);

			// Test where Shape00 should overlap Shape10
			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 1);

			SetShapeFilter(UnionProxy1, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 0, 3, EFilterFlags::All, 0b00010);
			SetShapeFilter(UnionProxy0, 1, 3, EFilterFlags::All, 0b01000);
			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 1);

			// Test where Shape01 should overlap both shapes on proxy1
			SetShapeFilter(UnionProxy1, 0, 1, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy1, 1, 2, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 0, 3, EFilterFlags::All, 0b01000);
			SetShapeFilter(UnionProxy0, 1, 3, EFilterFlags::All, 0b00110);
			Scene.TickSolver(Dt);
			CHECK(BroadPhase.GetNumBroadPhasePairs() == 1);
		}
	}
} // namespace Chaos
