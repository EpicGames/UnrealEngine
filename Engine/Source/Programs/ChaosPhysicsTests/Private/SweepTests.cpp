// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"
#include <catch2/matchers/catch_matchers_all.hpp>

#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/Sweeps.h"

namespace Chaos
{
	TEST_CASE("SweepSphereVsSphere", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const TSphere<double, 3> TestGeom(FVec3::ZeroVector, 5);
		const TSphere<double, 3> SweepGeom(FVec3::ZeroVector, 2);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(3.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(11, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(3.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(6, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, -8, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, 1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(3.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, -3, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(3.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 7, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, -7), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, 1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(3.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, -2)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, 13), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, -1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(3.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 8)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -X with diagonal hit")
		{
			// Use law of sines to figure out the positional offset for a 45 degree angle hit
			const FReal ZOffset = (TestGeom.GetRadiusf() + SweepGeom.GetRadiusf()) * FMath::Sin(FMath::DegreesToRadians(45)) / FMath::Sin(FMath::DegreesToRadians(90));
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3 + ZOffset), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FVec3 ExpectedNormal = FVec3(-1, 0, 1).GetSafeNormal();
			const FVec3 ExpectedPosition = TestGeomTransform.GetLocation() + ExpectedNormal * TestGeom.GetRadiusf();

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.05025, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(ExpectedPosition));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1.05 + Length + 7, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-5, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Deep Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(0, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Initial Overlap Behind")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(2, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-6, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(6, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("With Thickness")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestThickness = 1;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, TestThickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(2.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Zero Length Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult == false);
		}
		SECTION("Zero Length Sweep: Hit")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-5, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepSphereVsSphere - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FVec3 Dir(1, 0, 0);
		const FReal Length = 100;
		const TSphere<double, 3> TestGeom(FVec3::ZeroVector, 5);
		const TSphere<double, 3> SweepGeom(FVec3::ZeroVector, 2);
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());

		FReal ResultTime;
		FVec3 ResultPosition, ResultNormal, ResultFaceNormal;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-5, 2, 3), FRotation3::FromIdentity());
			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(0, 2, 3), FRotation3::FromIdentity());
			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}

	TEST_CASE("SweepSphereVsBox", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const TBox<double, 3> TestGeom(FVec3(-1), FVec3(1));
		const TSphere<double, 3> SweepGeom(FVec3::ZeroVector, 2);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(0, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(11, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, -8, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, 1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 1, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 3, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, -7), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, 1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 2)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, 13), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, -1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 4)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1.05 + Length + 3, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-1, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(0, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Deep Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(0.9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-2.9, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(0, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Initial Overlap Behind")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(2, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-2, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Rotation - X 45")
		{
			// Do a sweep along the local y axis, however rotate the entire test by 45 degrees along the x.
			const FVec3 GeomLocation = FVec3(1, 2, 3);
			const FRigidTransform3 TestGeomTransform(GeomLocation, FRotation3::FromAxisAngle(FVec3(1, 0, 0), PI * 0.25));
			const FVec3 WorldOffset = TestGeomTransform.TransformVectorNoScale(FVec3(0, 10, 0));
			const FRigidTransform3 SweepGeomTransform(GeomLocation + WorldOffset, FRotation3::FromIdentity());
			const FVec3 Dir = -WorldOffset.GetSafeNormal();
			const FVec3 ExpectedNormal = -Dir;
			const FVec3 ExpectedPosition = GeomLocation + ExpectedNormal; // Box has size 1

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(ExpectedPosition));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Zero Length Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult == false);
		}
		SECTION("Zero Length Sweep: Hit")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-1, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(0, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Edge Intersection")
		{
			// This test primarily is to show the face normal is correctly computed and it's different than the normal
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(3.99, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(8.8, 0.1));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(2, 3, 3), 0.1));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0), 0.1));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepSphereVsBox - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FVec3 Dir(1, 0, 0);
		const TBox<double, 3> TestGeom(FVec3(-1), FVec3(1));
		const TSphere<double, 3> SweepGeom(FVec3::ZeroVector, 2);
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());

		FReal ResultTime;
		FVec3 ResultPosition, ResultNormal, ResultFaceNormal;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Edge Hit")
		{
			const FVec3 Dir2 = FVec3(-1, 0, -1).GetSafeNormal();
			const FRigidTransform3 SweepGeomTransform(TestGeomTransform.GetLocation() - Dir2 * 10, FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir2, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Vertex Hit")
		{
			const FVec3 Dir2 = FVec3(-1, -1, -1).GetSafeNormal();
			const FRigidTransform3 SweepGeomTransform(TestGeomTransform.GetLocation() - Dir2 * 10, FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir2, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-1, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(0.9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}

	TEST_CASE("SweepSphereVsCapsule", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FCapsule TestGeom(FVec3(0, -1, 0), FVec3(0, 1, 0), 3);
		const TSphere<double, 3> SweepGeom(FVec3::ZeroVector, 2);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(11, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, -8, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, 1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, -2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 6, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, -7), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, 1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 0)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, 13), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, -1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 6)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -X with diagonal hit")
		{
			// Use law of sines to figure out the positional offset for a 45 degree angle hit
			const FReal ZOffset = (TestGeom.GetRadiusf() + SweepGeom.GetRadiusf()) * FMath::Sin(FMath::DegreesToRadians(45)) / FMath::Sin(FMath::DegreesToRadians(90));
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3 + ZOffset), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FVec3 ExpectedNormal = FVec3(-1, 0, 1).GetSafeNormal();
			const FVec3 ExpectedPosition = TestGeomTransform.GetLocation() + ExpectedNormal * TestGeom.GetRadiusf();

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.464466, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(ExpectedPosition));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1.05 + Length + 5, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-3, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Deep Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(0.9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-4.9, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Initial Overlap Behind")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(2, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-4, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sphere Rotation: 90 about X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(11, 2, 3), FRotation3::FromAxisAngle(FVec3(1, 0, 0), PI * 0.5));
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Capsule Rotation: 90 about X")
		{
			// Rotate 90 degrees along the x then do a sweep along the z axis. This is now the long axis of the capsule
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromAxisAngle(FVec3(1, 0, 0), PI * 0.5));
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, -7), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, 1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, -1)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Capsule Rotation: 45 about X")
		{
			// Do a sweep along the local y axis, however rotate the entire test by 45 degrees along the x.
			const FVec3 GeomLocation = FVec3(1, 2, 3);
			const FRigidTransform3 TestGeomTransform(GeomLocation, FRotation3::FromAxisAngle(FVec3(1, 0, 0), PI * 0.25));
			const FVec3 WorldOffset = TestGeomTransform.TransformVectorNoScale(FVec3(0, 10, 0));
			const FRigidTransform3 SweepGeomTransform(GeomLocation + WorldOffset, FRotation3::FromIdentity());
			const FVec3 Dir = -WorldOffset.GetSafeNormal();
			const FVec3 ExpectedNormal = -Dir;
			const FVec3 ExpectedPosition = GeomLocation + ExpectedNormal * 4; // Offset on axis is radius + half extent = 3 + 1 = 4

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(ExpectedPosition));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("With Thickness")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestThickness = 1;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, TestThickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Zero Length Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult == false);
		}
		SECTION("Zero Length Sweep: Hit")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-3, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepSphereVsCapsule - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FVec3 Dir(1, 0, 0);
		const FReal Length = 100;
		const FReal Thickness = 0;
		const FCapsule TestGeom(FVec3(0, -1, 0), FVec3(0, 1, 0), 3);
		const TSphere<double, 3> SweepGeom(FVec3::ZeroVector, 2);
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());

		FReal ResultTime;
		FVec3 ResultPosition, ResultNormal, ResultFaceNormal;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-3, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(0.9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}

	TEST_CASE("SweepBoxVsSphere", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const TSphere<double, 3> TestGeom(FVec3::ZeroVector, 2);
		const TBox<double, 3> SweepGeom(FVec3(-1), FVec3(1));

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1.05 + Length + 3, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-1, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Edge Intersection")
		{
			// This test primarily is to show the face normal is correctly computed and it's different than the normal
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(3.99, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(8.8, 0.1));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(3, 2.2, 3), 0.1));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0), 0.1));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0), 0.1));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepBoxVsBox", "[Chaos][Sweeps][unit]")
	{
		// TODO
	}

	TEST_CASE("SweepBoxVsBox - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const TBox<double, 3> TestGeom(FVec3(-1), FVec3(1));
		const TBox<double, 3> SweepGeom(FVec3(-1), FVec3(1));
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
		const FVec3 Dir(1, 0, 0);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-2.5, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(0.9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}

	TEST_CASE("SweepBoxVsCapsule", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FCapsule XCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 2);
		const FCapsule YCapsule(FVec3(0, -1, 0), FVec3(0, 1, 0), 2);
		const FCapsule ZCapsule(FVec3(0, 0, -1), FVec3(0, 0, 1), 2);
		const TBox<double, 3> SweepGeom(FVec3(-1), FVec3(1));

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(11, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(4, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, -8, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, 1, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, -1, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, -1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Y")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 5, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 1, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, -7), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, 1);

			const bool bResult = SweepQuery(ZCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 0)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From +Z")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, 13), FRotation3::FromIdentity());
			const FVec3 Dir(0, 0, -1);

			const bool bResult = SweepQuery(ZCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 6)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, 1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1.05 + Length + 7, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-2, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Deep Initial Overlap")
		{
			const FCapsule SmallXCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 0.5);
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-0.9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(SmallXCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-0.6, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-0.5, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Zero Length Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult == false);
		}
		SECTION("Zero Length Sweep: Hit")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-2, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Edge Intersection")
		{
			// This test primarily is to show the face normal is correctly computed and it's different than the normal
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(3.99, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.8, 0.1));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(3, 3.2, 3), 0.1));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0), 0.1));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(1, 0, 0), 0.1));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Rotation")
		{
			// Rotate the box 45 degrees so we get an edge intersection and the capsule 90 degrees
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromAxisAngle(FVec3(0, 0, 1), PI * 0.5));
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromAxisAngle(FVec3(0, 0, 1), PI * 0.25));
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.58578, 0.01)); // 10 - radius(2) - sqrt(1 + 1) ~= 6.58
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepBoxVsCapsule - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FCapsule XCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 2);
		const TBox<double, 3> SweepGeom(FVec3(-1), FVec3(1));
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
		const FVec3 Dir(1, 0, 0);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-2, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(XCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FCapsule SmallXCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 0.5);
			const FRigidTransform3 SweepGeomTransform(FVec3(-0.9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(SmallXCapsule, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}

	TEST_CASE("SweepCapsuleVsSphere", "[Chaos][Sweeps][unit]")
	{
		// Note: This sweep is minimally tested as it's just a flipped version of SphereVsCapsule. This is included to make sure the basics are correct.
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const TSphere<double, 3> TestGeom(FVec3::ZeroVector, 2);
		const FCapsule SweepGeom(FVec3(0, -1, 0), FVec3(0, 1, 0), 3);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep From -X with diagonal hit")
		{
			// Use law of sines to figure out the positional offset for a 45 degree angle hit
			const FReal ZOffset = (TestGeom.GetRadiusf() + SweepGeom.GetRadiusf()) * FMath::Sin(FMath::DegreesToRadians(45)) / FMath::Sin(FMath::DegreesToRadians(90));
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3 + ZOffset), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FVec3 ExpectedNormal = FVec3(-1, 0, 1).GetSafeNormal();
			const FVec3 ExpectedPosition = TestGeomTransform.GetLocation() + ExpectedNormal * TestGeom.GetRadiusf();

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.464466, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(ExpectedPosition));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(ExpectedNormal));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sphere Rotation: 90 about X")
		{
			// Test a rotation on the above -X test. Nothing should change
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromAxisAngle(FVec3(1, 0, 0), PI * 0.5));
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Capsule Rotation: 90 about X")
		{
			// Rotate 90 degrees along the x then do a sweep along the z axis. This is now the long axis of the capsule
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1, 2, -7), FRotation3::FromAxisAngle(FVec3(1, 0, 0), PI * 0.5));
			const FVec3 Dir(0, 0, 1);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(1, 2, 1)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 0, -1)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Miss (Behind Sweep)")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(7, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-3, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Zero Length Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult == false);
		}
		SECTION("Zero Length Sweep: Hit")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-3, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);
			const FReal TestLength = 0;

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, TestLength, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepCapsuleVsSphere - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FVec3 Dir(1, 0, 0);
		const FReal Length = 100;
		const FReal Thickness = 0;
		const FCapsule SweepGeom(FVec3(0, -1, 0), FVec3(0, 1, 0), 3);
		const TSphere<double, 3> TestGeom(FVec3::ZeroVector, 2);
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());

		FReal ResultTime;
		FVec3 ResultPosition, ResultNormal, ResultFaceNormal;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-3, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(0.9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(TestGeom, TestGeomTransform, SweepGeom, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}

	TEST_CASE("SweepCapsuleVsBox", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FCapsule XCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 2);
		const FCapsule YCapsule(FVec3(0, -1, 0), FVec3(0, 1, 0), 2);
		const FCapsule ZCapsule(FVec3(0, 0, -1), FVec3(0, 0, 1), 2);
		const TBox<double, 3> TestGeom(FVec3(-1), FVec3(1));

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Sweep From -X")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(0, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1.05 + Length + 7, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(-1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-2, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-1.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(0, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Edge Intersection")
		{
			// This test primarily is to show the face normal is correctly computed and it's different than the normal
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(3.99, 12, 3), FRotation3::FromIdentity());
			const FVec3 Dir(0, -1, 0);

			const bool bResult = SweepQuery(TestGeom, TestGeomTransform, YCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(7.8, 0.1));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(2, 3, 3), 0.1));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(1, 0, 0), 0.1));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(0, 1, 0), 0.1));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepCapsuleVsCapsule", "[Chaos][Sweeps][unit]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FCapsule XCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 2);
		const FCapsule YCapsule(FVec3(0, -1, 0), FVec3(0, 1, 0), 2);
		const FCapsule ZCapsule(FVec3(0, 0, -1), FVec3(0, 0, 1), 2);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		SECTION("Parallel Edge Edge")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, YCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition.X, Catch::Matchers::WithinRel(-1, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Perpendicular Edge Edge")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, ZCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Cap vs Edge 1")
		{
			// Sweep an x capsule so it hits the y capsule in the center
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(5.0, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Cap vs Edge 2")
		{
			// Sweep a z capsule so its cap barely hits the y capsule in the center
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 0.1), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, ZCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(6.48, 0.01));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-.759, 2, 2.049), 0.001));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-.880, 0, -.474), 0.001));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-.880, 0, -.474), 0.001));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Cap vs Cap 1")
		{
			// Sweep an x capsule against another x capsule
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(XCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(4, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-2, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Sweep: Past max distance")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(1 - Length - 6, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Sweep: Miss")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(!bResult);
		}
		SECTION("Shallow Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-3.9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-0.1, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
		SECTION("Deep Initial Overlap")
		{
			const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
			const FRigidTransform3 SweepGeomTransform(FVec3(-0.1, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			const bool bResult = SweepQuery(YCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			CHECK(bResult);
			CHECK_THAT(ResultTime, Catch::Matchers::WithinRel(-3.9, UE_DOUBLE_KINDA_SMALL_NUMBER));
			CHECK_THAT(ResultPosition, Catch::ApproxEq(FVec3(-1, 2, 3)));
			CHECK_THAT(ResultNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK_THAT(ResultFaceNormal, Catch::ApproxEq(FVec3(-1, 0, 0)));
			CHECK(ResultFaceIndex == INDEX_NONE);
		}
	}

	TEST_CASE("SweepCapsuleVsCapsule - Perf", "[Chaos][Sweeps][!benchmark]")
	{
		const bool bComputeMTD = true;
		const FReal Thickness = 0;
		const FReal Length = 100;
		const FCapsule XCapsule(FVec3(-1, 0, 0), FVec3(1, 0, 0), 2);
		const FCapsule YCapsule(FVec3(0, -1, 0), FVec3(0, 1, 0), 2);
		const FCapsule ZCapsule(FVec3(0, 0, -1), FVec3(0, 0, 1), 2);
		const FRigidTransform3 TestGeomTransform(FVec3(1, 2, 3), FRotation3::FromIdentity());
		const FVec3 Dir(1, 0, 0);

		FReal ResultTime;
		FVec3 ResultPosition;
		FVec3 ResultNormal;
		FVec3 ResultFaceNormal = FVec3::ZeroVector;
		int32 ResultFaceIndex = INDEX_NONE;

		BENCHMARK("Basic Hit")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(XCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Perpendicular Edge Edge")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 2, 3), FRotation3::FromIdentity());
			const FVec3 Dir(1, 0, 0);

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(YCapsule, TestGeomTransform, ZCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Miss")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-9, 10, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(XCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Shallow Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-3.9, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(YCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
		BENCHMARK("Deep Initial Overlap")
		{
			const FRigidTransform3 SweepGeomTransform(FVec3(-0.1, 2, 3), FRotation3::FromIdentity());

			bool bResult = false;
			for (int32 I = 0; I < 1000; ++I)
			{
				bResult |= SweepQuery(XCapsule, TestGeomTransform, XCapsule, SweepGeomTransform, Dir, Length, ResultTime, ResultPosition, ResultNormal, ResultFaceIndex, ResultFaceNormal, Thickness, bComputeMTD);
			}
			return bResult;
		};
	}
} // namespace Chaos
